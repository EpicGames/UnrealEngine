// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderCompilerCommonInternal.h"
#include "HlslParserInternal.h"
#include "ShaderParameterParser.h"


namespace UE::ShaderCompilerCommon
{

bool RemoveUnusedOutputs(
	const FShaderCompilerInput& CompilerInput,
	FShaderCompilerOutput& CompilerOutput,
	FString& InOutSourceCode,
	FString& InOutEntryPoint,
	TConstArrayView<FStringView> InUsedOutputs,
	TConstArrayView<FStringView> InExceptions,
	TConstArrayView<UE::HlslParser::FScopedDeclarations> InScopedDeclarations)
{
	TArray<FString> HlslParserErrors;
	if (!UE::HlslParser::RemoveUnusedOutputs(InOutSourceCode, InUsedOutputs, InExceptions, InScopedDeclarations, InOutEntryPoint, HlslParserErrors))
	{
		CompilerOutput.bInterpolatorRemovalFailed = true;
		CompilerOutput.Errors.Add(FShaderCompilerError(FString::Printf(TEXT("Failed to remove unused outputs from shader: %s"), *CompilerInput.GenerateShaderName())));
		for (FString& Error : HlslParserErrors)
		{
			CompilerOutput.Errors.Add(FShaderCompilerError(MoveTemp(Error)));
		}
		return false;
	}
	else
	{
		// Unused interpolators were removed, so HlslParser generated new HLSL with a new entry point that wraps the original entry point.
		CompilerOutput.ModifiedShaderSource = InOutSourceCode;
		CompilerOutput.ModifiedEntryPointName = InOutEntryPoint;
		return true;
	}
}

bool RemoveUnusedInputs(
	const FShaderCompilerInput& CompilerInput,
	FShaderCompilerOutput& CompilerOutput,
	FStringView InSourceCode,
	FStringView InEntryPointName,
	TConstArrayView<FString> InUsedInputs,
	TConstArrayView<UE::HlslParser::FScopedDeclarations> InScopedDeclarations)
{
	TArray<FString> HlslParserErrors;
	FString ModifiedSourceCode(InSourceCode);
	FString ModifiedEntryPoint(InEntryPointName);
	const TArray<FStringView> InUsedInputsView(MakeArrayView(InUsedInputs));
	if (!UE::HlslParser::RemoveUnusedInputs(ModifiedSourceCode, InUsedInputsView, InScopedDeclarations, ModifiedEntryPoint, HlslParserErrors))
	{
		CompilerOutput.bInterpolatorRemovalFailed = true;
		CompilerOutput.Errors.Add(FShaderCompilerError(FString::Printf(TEXT("Failed to remove unused inputs from shader: %s"), *CompilerInput.GenerateShaderName())));
		for (FString& Error : HlslParserErrors)
		{
			CompilerOutput.Errors.Add(FShaderCompilerError(MoveTemp(Error)));
		}
		return false;
	}
	else
	{
		// Unused interpolators were removed, so HlslParser generated new HLSL with a new entry point that wraps the original entry point.
		CompilerOutput.ModifiedShaderSource = MoveTemp(ModifiedSourceCode);
		CompilerOutput.ModifiedEntryPointName = MoveTemp(ModifiedEntryPoint);
		return true;
	}
}

#if PLATFORM_WINDOWS
class FDxcMalloc final : public IMalloc
{
	std::atomic<ULONG> RefCount{ 1 };

public:

	// IMalloc

	void* STDCALL Alloc(SIZE_T cb) override
	{
		cb = FMath::Max(SIZE_T(1), cb);
		return FMemory::Malloc(cb);
	}

	void* STDCALL Realloc(void* pv, SIZE_T cb) override
	{
		cb = FMath::Max(SIZE_T(1), cb);
		return FMemory::Realloc(pv, cb);
	}

	void STDCALL Free(void* pv) override
	{
		return FMemory::Free(pv);
	}

	SIZE_T STDCALL GetSize(void* pv) override
	{
		return FMemory::GetAllocSize(pv);
	}

	int STDCALL DidAlloc(void* pv) override
	{
		return 1; // assume that all allocation queries coming from DXC belong to our allocator
	}

	void STDCALL HeapMinimize() override
	{
		// nothing
	}

	// IUnknown

	ULONG STDCALL AddRef() override
	{
		return ++RefCount;
	}

	ULONG STDCALL Release() override
	{
		check(RefCount > 0);
		return --RefCount;
	}

	HRESULT STDCALL QueryInterface(REFIID iid, void** ppvObject) override
	{
		checkNoEntry(); // We do not expect or support QI on DXC allocator replacement
		return ERROR_NOINTERFACE;
	}
};

IMalloc* GetDxcMalloc()
{
	static FDxcMalloc Instance;
	return &Instance;
}
#endif // PLATFORM_WINDOWS


// Go through the specialization constants in the SPIRV to match them to what was expected in the build env.
// Evaluate all the different combinations of the spec consts to find the resources that are conditional on specific values.
FSpecializationConstantData ProcessSpirvSpecializationConstants(
	const FShaderCompilerInput& Input, 
	CrossCompiler::FShaderConductorContext& CompilerContext,
	spv_reflect::ShaderModule& Reflection,
	const FSpirvReflectBindings& UnspecializedBindings,
	FShaderCompilerOutput& Output)
{
	auto& SpecializationConstantValues = Input.Environment.SpecializationConstantValues;
	const int32 TotalSpecializationCount = SpecializationConstantValues.Num();

	// Bail if specialization constants aren't used
	if (TotalSpecializationCount <= 0)
	{
		return FSpecializationConstantData();
	}

	// Put the total count of spec constants
	uint32 SPIRVSpecializationCount = 0;
	Reflection.EnumerateSpecializationConstants(&SPIRVSpecializationCount, nullptr);
	TArray<SpvReflectSpecializationConstant*> SPIRVSpecConsts;
	SPIRVSpecConsts.SetNumZeroed(SPIRVSpecializationCount);
	Reflection.EnumerateSpecializationConstants(&SPIRVSpecializationCount, SPIRVSpecConsts.GetData());

	// Read the array of expected specialization constants from the build environment
	checkf((uint32)TotalSpecializationCount >= SPIRVSpecializationCount,
		TEXT("SPIRV specialization count (%u) is greater than expected (%d) by build environment."),
		SPIRVSpecializationCount, TotalSpecializationCount);

	// SPIRV-Tool passes to run to evaluate constants
	// No need for "set-spec-const-default-value", we'll use SPIRVReflect
	const ANSICHAR* OptimizerConfig[] = {

		// Freeze the spec constants
		"--freeze-spec-const",
		"--fold-spec-const-op-composite",

		// Remove dead code from the frozen constants
		"--eliminate-dead-code-aggressive",
		"--eliminate-dead-inserts",
		"--eliminate-dead-branches",
		"--eliminate-dead-members",
		"--eliminate-dead-variables",
		"--eliminate-dead-const",

		// Full -O passes (note: could probably be reduced)
		"--eliminate-dead-branches",
		"--merge-return",
		"--inline-entry-points-exhaustive",
		"--eliminate-dead-functions",
		"--eliminate-dead-code-aggressive",
		"--private-to-local",
		"--eliminate-local-single-block",
		"--eliminate-local-single-store",
		"--eliminate-dead-code-aggressive",
		"--scalar-replacement",
		"--convert-local-access-chains",
		"--eliminate-local-single-block",
		"--eliminate-local-single-store",
		"--eliminate-dead-code-aggressive",
		"--eliminate-local-multi-store",
		"--eliminate-dead-code-aggressive",
		"--ccp",
		"--eliminate-dead-code-aggressive",
		"--loop-unroll",
		"--eliminate-dead-branches",
		"--redundancy-elimination",
		"--combine-access-chains",
		"--simplify-instructions",
		"--scalar-replacement",
		"--convert-local-access-chains",
		"--eliminate-local-single-block",
		"--eliminate-local-single-store",
		"--eliminate-dead-code-aggressive",
		"--ssa-rewrite",
		"--eliminate-dead-code-aggressive",
		"--vector-dce",
		"--eliminate-dead-inserts",
		"--eliminate-dead-branches",
		"--simplify-instructions",
		"--if-conversion",
		"--copy-propagate-arrays",
		"--reduce-load-size",
		"--eliminate-dead-code-aggressive",
		"--merge-blocks",
		"--redundancy-elimination",
		"--eliminate-dead-branches",
		"--merge-blocks",
		"--simplify-instructions",
		"--eliminate-dead-members",
		"--eliminate-local-single-store",
		"--merge-blocks",
		"--eliminate-local-multi-store",
		"--redundancy-elimination",
		"--simplify-instructions",
		"--eliminate-dead-code-aggressive",
		"--cfg-cleanup"
	};

	const bool bIsBindless = Input.IsBindlessEnabled();

	// Map of resource names to hashes of spec const values to exclude the resource from
	TMap<FString, TArray<uint32>> ExcludedResources;
	auto DiffResourceArrays = [&](const uint32 SpecHash, const TArray<SpvReflectDescriptorBinding*>& UnspecArray, const TArray<SpvReflectDescriptorBinding*>& SpecArray)
		{
			for (SpvReflectDescriptorBinding* UnspecBinding : UnspecArray)
			{
				if (!SpecArray.FindByPredicate([UnspecBinding](SpvReflectDescriptorBinding* SpecBinding) {
					return FCStringAnsi::Strcmp(UnspecBinding->name, SpecBinding->name) == 0;
					}))
				{
					TArray<uint32>& Hashes = ExcludedResources.FindOrAdd(UnspecBinding->name);
					Hashes.Add(SpecHash);
				}
			}
		};

	auto DiffBindlessResources = [&](const uint32 SpecHash, const SpvReflectBlockVariable& UnspecGlobals, const SpvReflectBlockVariable& SpecGlobals)
		{
			TConstArrayView<SpvReflectBlockVariable> UnspecMembers(UnspecGlobals.members, UnspecGlobals.member_count);
			TConstArrayView<SpvReflectBlockVariable> SpecMembers(SpecGlobals.members, SpecGlobals.member_count);
			for (const SpvReflectBlockVariable& UnspecMember : UnspecMembers)
			{
				const FString MemberName(UnspecMember.name);
				FStringView AdjustedMemberName(MemberName);
				const EShaderParameterType BindlessParameterType = FShaderParameterParser::ParseAndRemoveBindlessParameterPrefix(AdjustedMemberName);
				if (BindlessParameterType != EShaderParameterType::LooseData)
				{
					if (!SpecMembers.FindByPredicate([&UnspecMember](const SpvReflectBlockVariable& SpecMember) {
						return (FCStringAnsi::Strcmp(UnspecMember.name, SpecMember.name) == 0);
						}))
					{
						TArray<uint32>& Hashes = ExcludedResources.FindOrAdd(FString(AdjustedMemberName));
						Hashes.Add(SpecHash);
					}
				}
			}
		};

	auto ProcessFullSpecialization = [&](const TArray<int32>& SpecValues)
		{
			// Set the default values to the combination we are evaluating
			for (SpvReflectSpecializationConstant* SpirvSpecConst : SPIRVSpecConsts)
			{
				checkf(SpirvSpecConst->default_value_size == 4, TEXT("All specialization constants should current be 32bit wide."));
				checkf(SpirvSpecConst->constant_id < (uint32)SpecializationConstantValues.Num(), TEXT("Specialization constants index out of bounds."));

				*((int32*)SpirvSpecConst->default_value) = SpecValues[SpirvSpecConst->constant_id];
			}

			TArray<uint32> SpirvCopy(Reflection.GetCode(), Reflection.GetCodeSize() / sizeof(uint32));

			if (CompilerContext.OptimizeSpirv(SpirvCopy, OptimizerConfig, UE_ARRAY_COUNT(OptimizerConfig)))
			{
				// Return code reflection if requested for shader analysis
				if (Input.Environment.CompilerFlags.Contains(CFLAG_OutputAnalysisArtifacts))
				{
					FGenericShaderStat SpecConstReflection;
					if (CrossCompiler::FShaderConductorContext::Disassemble(CrossCompiler::EShaderConductorIR::Spirv, SpirvCopy.GetData(), SpirvCopy.NumBytes(), SpecConstReflection))
					{
						FString SpecValString;
						for (int32 SpecId : SpecValues)
						{
							SpecValString.Append(FString::Printf(TEXT(".%d"), SpecId));
						}
						SpecConstReflection.StatName = FName(SpecConstReflection.StatName.ToString() + SpecValString);

						Output.ShaderStatistics.Add(MoveTemp(SpecConstReflection));
					}
				}

				// Parse the new SPIRV (no need to copy the SPIRV again)
				spv_reflect::ShaderModule EvalReflection(SpirvCopy.NumBytes(), SpirvCopy.GetData(), SPV_REFLECT_MODULE_FLAG_NO_COPY);

				// Grab resources in SPIRV with spec constats evaluated
				FSpirvReflectBindings EvalBindings;
				EvalBindings.GatherDescriptorBindings(EvalReflection);

				const uint32 SpecHash = GetTypeHash(SpecValues);

				if (bIsBindless)
				{
					const SpvReflectDescriptorBinding* const* UnspecializedGlobals = 
						UnspecializedBindings.UniformBuffers.FindByPredicate([](const SpvReflectDescriptorBinding* Binding) {
							return FCStringAnsi::Strcmp("$Globals", Binding->name) == 0;
						});

					SpvReflectDescriptorBinding** SpecializedGlobals =
						EvalBindings.UniformBuffers.FindByPredicate([](SpvReflectDescriptorBinding* Binding) {
							return FCStringAnsi::Strcmp("$Globals", Binding->name) == 0;
						});

					if (UnspecializedGlobals && SpecializedGlobals)
					{
						DiffBindlessResources(SpecHash, (*UnspecializedGlobals)->block, (*SpecializedGlobals)->block);
					}
				}
				else
				{
					DiffResourceArrays(SpecHash, UnspecializedBindings.UniformBuffers, EvalBindings.UniformBuffers);
					DiffResourceArrays(SpecHash, UnspecializedBindings.Samplers, EvalBindings.Samplers);
					DiffResourceArrays(SpecHash, UnspecializedBindings.TextureSRVs, EvalBindings.TextureSRVs);
					DiffResourceArrays(SpecHash, UnspecializedBindings.TextureUAVs, EvalBindings.TextureUAVs);
					DiffResourceArrays(SpecHash, UnspecializedBindings.TBufferSRVs, EvalBindings.TBufferSRVs);
					DiffResourceArrays(SpecHash, UnspecializedBindings.TBufferUAVs, EvalBindings.TBufferUAVs);
					DiffResourceArrays(SpecHash, UnspecializedBindings.SBufferSRVs, EvalBindings.SBufferSRVs);
					DiffResourceArrays(SpecHash, UnspecializedBindings.SBufferUAVs, EvalBindings.SBufferUAVs);
					DiffResourceArrays(SpecHash, UnspecializedBindings.AccelerationStructures, EvalBindings.AccelerationStructures);
				}
			}
		};

	// Go through every possible combination of the possible values for the spec contants
	TArray<int32> SpecValues;
	SpecValues.SetNumZeroed(TotalSpecializationCount);
	const TFunction<void(int32)> ProcessDimension = [&](const int32 DimensionIndex)
		{
			for (int32 SpecIndex = 0; SpecIndex < SpecializationConstantValues[DimensionIndex].Num(); ++SpecIndex)
			{
				SpecValues[DimensionIndex] = SpecializationConstantValues[DimensionIndex][SpecIndex];

				if (DimensionIndex < SpecializationConstantValues.Num() - 1)
				{
					ProcessDimension(DimensionIndex + 1);
				}
				else
				{
					ProcessFullSpecialization(SpecValues);
				}
			}
		};

	// Start processing at the first spec const
	ProcessDimension(0);

	// Create a new ShaderModule
	// Iterate over every possible combination
	//   Set the values using Reflection.m_module.spec_constants->default_value
	//   Run the FoldSpecConstantOpAndCompositePass pass to fold the OpSpecConstantOp
	//   Run a subset of optimization passes to make sure unused resources are removed
	//   Run GatherSpirvReflectionBindings and compare the resources
	//     Mark the unused ones in the Parameter map

	FSpecializationConstantData SpecializationConstantData;
	SpecializationConstantData.Values = Input.Environment.SpecializationConstantValues;

	TMap<uint32, TArray<uint32>> ExclusionMap;
	const TMap<FString, FParameterAllocation>& ParameterMap = Output.ParameterMap.ParameterMap;
	for (auto& Pair : ExcludedResources)
	{
		if (const FParameterAllocation* Param = ParameterMap.Find(Pair.Key))
		{
			TArray<uint32>& Hashes = ExclusionMap.FindOrAdd(Param->BaseIndex);
			Hashes = MoveTemp(Pair.Value);
		}
	}
	SpecializationConstantData.SetExclusionMap(MoveTemp(ExclusionMap));
	SpecializationConstantData.Sort();
	return SpecializationConstantData;
}

} // UE::ShaderCompilerCommon

