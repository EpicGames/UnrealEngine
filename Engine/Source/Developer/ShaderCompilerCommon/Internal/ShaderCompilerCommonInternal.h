// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ShaderCompilerCore.h"
#include "HlslParserInternal.h"
#include "SpirvReflectCommon.h"
#include "ShaderConductorContext.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include <objidl.h>  // For IMalloc
#include "Windows/HideWindowsPlatformTypes.h"
#endif

namespace UE::ShaderCompilerCommon
{

	extern SHADERCOMPILERCOMMON_API bool RemoveUnusedOutputs(
		const FShaderCompilerInput& CompilerInput,
		FShaderCompilerOutput& CompilerOutput,
		FString& InOutSourceCode,
		FString& InOutEntryPoint,
		TConstArrayView<FStringView> InUsedOutputs,
		TConstArrayView<FStringView> InExceptions,
		TConstArrayView<UE::HlslParser::FScopedDeclarations> InScopedDeclarations
	);

	extern SHADERCOMPILERCOMMON_API bool RemoveUnusedInputs(
		const FShaderCompilerInput& CompilerInput,
		FShaderCompilerOutput& CompilerOutput,
		FStringView InSourceCode,
		FStringView InEntryPointName,
		TConstArrayView<FString> InUsedInputs,
		TConstArrayView<UE::HlslParser::FScopedDeclarations> InScopedDeclarations = {}
	);

#if PLATFORM_WINDOWS
	// Get an IMalloc to overload DXC's allocator with our own in Windows
	extern SHADERCOMPILERCOMMON_API IMalloc* GetDxcMalloc();
#endif

	extern SHADERCOMPILERCOMMON_API FSpecializationConstantData ProcessSpirvSpecializationConstants(
		const FShaderCompilerInput& Input,
		CrossCompiler::FShaderConductorContext& CompilerContext,
		spv_reflect::ShaderModule& Reflection,
		const FSpirvReflectBindings& UnspecializedBindings,
		FShaderCompilerOutput& Output);
}

