// Copyright Epic Games, Inc. All Rights Reserved.

#include "DXCWrapper.h"
#include "HAL/PlatformProcess.h"
#include "HAL/FileManager.h"
#include "Templates/RefCounting.h"
#include "Misc/Paths.h"
#include "Hash/xxhash.h"

#if PLATFORM_WINDOWS
#include "Windows/WindowsHWrapper.h"
#include "ImageHlp.h"
#endif // PLATFORM_WINDOWS

static TRefCountPtr<FDllHandle> GDxilHandle;
static TRefCountPtr<FDllHandle> GDxcHandle;
static TRefCountPtr<FDllHandle> GShaderConductorHandle;

#if PLATFORM_WINDOWS
BOOL __stdcall DigestHash(DIGEST_HANDLE Handle, PBYTE Data, DWORD Length)
{
	FXxHash64Builder& Hasher = *reinterpret_cast<FXxHash64Builder*>(Handle);
	Hasher.Update(Data, Length);
	return true;
}
#endif

static uint64 GetLoadedModuleVersion(const TCHAR* ModuleName)
{
#if PLATFORM_WINDOWS
	HMODULE ModuleDll = ::GetModuleHandleW(ModuleName);
	if (ModuleDll == nullptr)
	{
		return 0;
	}

	TCHAR DllPath[4096] = {};
	int32 PathLen = ::GetModuleFileNameW(ModuleDll, DllPath, UE_ARRAY_COUNT(DllPath));
	if (PathLen == 0)
	{
		return 0;
	}
	check(PathLen < UE_ARRAY_COUNT(DllPath));

	HANDLE DllHandle = ::CreateFileW(DllPath, GENERIC_READ, FILE_SHARE_READ | FILE_SHARE_WRITE, nullptr, OPEN_EXISTING, 0, nullptr);
	FXxHash64Builder FileHashBuilder;
	// ImageHlp APIs are not threadsafe; need a mutex around all calls.
	// At the moment this is the only usage of said APIs in the engine; if other usage 
	// is added this should be moved to a shared utility.
	static FCriticalSection ImageHlpCs;
	{
		FScopeLock Lock(&ImageHlpCs);
		bool bResult = ImageGetDigestStream(
			DllHandle,
			CERT_PE_IMAGE_DIGEST_ALL_IMPORT_INFO | CERT_PE_IMAGE_DIGEST_RESOURCES,
			DigestHash,
			reinterpret_cast<DIGEST_HANDLE>(&FileHashBuilder));
		check(bResult);
	}

	CloseHandle(DllHandle);

	return FileHashBuilder.Finalize().Hash;
#else // PLATFORM_WINDOWS
	return 0;
#endif // PLATFORM_WINDOWS
}

FDllHandle::FDllHandle(const TCHAR* InFilename)
{
#if PLATFORM_WINDOWS
	check(InFilename && *InFilename);
#if  PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
	FString ShaderConductorDir = FPaths::EngineDir() / TEXT("Binaries/ThirdParty/ShaderConductor/WinArm64");
#else
	FString ShaderConductorDir = FPaths::EngineDir() / TEXT("Binaries/ThirdParty/ShaderConductor/Win64");
#endif
	FString ModulePath = ShaderConductorDir / InFilename;
	Handle = FPlatformProcess::GetDllHandle(*ModulePath);
	checkf(Handle, TEXT("Failed to load module: %s"), *ModulePath);
#endif
}

FDllHandle::~FDllHandle()
{
	if (Handle)
	{
		FPlatformProcess::FreeDllHandle(Handle);
		Handle = nullptr;
	}
}


FDxcModuleWrapper::FDxcModuleWrapper()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDxcModuleWrapper::FDxcModuleWrapper);

	if (!GDxcHandle.IsValid())
	{
		GDxilHandle = new FDllHandle(TEXT("dxil.dll"));
		GDxcHandle = new FDllHandle(TEXT("dxcompiler.dll"));
	}
	else
	{
		GDxilHandle->AddRef();
		GDxcHandle->AddRef();
	}

	static uint64 DxcVersion = GetLoadedModuleVersion(TEXT("dxcompiler.dll"));

	// If dxil.dll is present, it's automatically loaded by dxcompiler.dll and used for validation and signing.
	// If dxil.dll is not present, shaders will silently be unsigned and will fail to load outside of development environment.
	// This must be taken into account when computing DDC key for D3D shaders.
	static uint64 DxilVersion = GetLoadedModuleVersion(TEXT("dxil.dll"));

	ModuleVersionHash = HashCombine(GetTypeHash(DxcVersion), GetTypeHash(DxilVersion));
}

FDxcModuleWrapper::~FDxcModuleWrapper()
{
	GDxcHandle.SafeRelease();
	GDxilHandle.SafeRelease();
}


FShaderConductorModuleWrapper::FShaderConductorModuleWrapper()
{
	if (!GShaderConductorHandle.IsValid())
	{
		GShaderConductorHandle = new FDllHandle(TEXT("ShaderConductor.dll"));
	}
	else
	{
		GShaderConductorHandle->AddRef();
	}

	static uint64 DllVersion = GetLoadedModuleVersion(TEXT("ShaderConductor.dll"));

	ModuleVersionHash = HashCombine(GetTypeHash(DllVersion), FDxcModuleWrapper::GetModuleVersionHash());
}

FShaderConductorModuleWrapper::~FShaderConductorModuleWrapper()
{
	GShaderConductorHandle.SafeRelease();
}
