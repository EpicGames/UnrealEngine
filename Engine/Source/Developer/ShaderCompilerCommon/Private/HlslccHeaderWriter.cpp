// Copyright Epic Games, Inc. All Rights Reserved.

#include "HlslccHeaderWriter.h"
#include "ShaderConductorContext.h"
#include "SpirvCommon.h"

THIRD_PARTY_INCLUDES_START
#include "spirv_reflect.h"
THIRD_PARTY_INCLUDES_END


namespace CrossCompiler
{

	template <typename... Types>
	static void MetaDataPrintf(FString& MetaData, UE::Core::TCheckedFormatString<FString::FmtCharType, Types...> Fmt, Types... Args)
	{
		if (!MetaData.IsEmpty())
		{
			MetaData += TEXT(",");
		}
		MetaData += FString::Printf(Fmt, Args...);
	}

	void FHlslccHeaderWriter::WriteSourceInfo(const TCHAR* VirtualSourceFilePath, const TCHAR* EntryPointName)
	{
		check(VirtualSourceFilePath != nullptr);
		check(EntryPointName != nullptr);
		Strings.SourceInfo = FString::Printf(TEXT("%s:%s"), VirtualSourceFilePath, EntryPointName);
	}

	void FHlslccHeaderWriter::WriteCompilerInfo(const TCHAR* CompilerName)
	{
		check(CompilerName != nullptr);
		Strings.CompilerInfo = CompilerName;
	}

	void FHlslccHeaderWriter::WriteInputAttribute(const SpvReflectInterfaceVariable& Attribute)
	{
		WriteIOAttribute(Strings.InputAttributes, Attribute, /*bIsInput:*/ true);
	}

	void FHlslccHeaderWriter::WriteInputAttribute(const TCHAR* AttributeName, const TCHAR* TypeSpecifier, int32 Location, bool bLocationPrefix, bool bLocationSuffix)
	{
		WriteIOAttribute(Strings.InputAttributes, AttributeName, TypeSpecifier, Location, bLocationPrefix, bLocationSuffix);
	}

	void FHlslccHeaderWriter::WriteOutputAttribute(const SpvReflectInterfaceVariable& Attribute)
	{
		WriteIOAttribute(Strings.OutputAttributes, Attribute, /*bIsInput:*/ false);
	}

	void FHlslccHeaderWriter::WriteOutputAttribute(const TCHAR* AttributeName, const TCHAR* TypeSpecifier, int32 Location, bool bLocationPrefix, bool bLocationSuffix)
	{
		WriteIOAttribute(Strings.OutputAttributes, AttributeName, TypeSpecifier, Location, bLocationPrefix, bLocationSuffix);
	}

	static void ConvertMetaDataTypeSpecifierPrimary(const SpvReflectTypeDescription& TypeSpecifier, FString& OutTypeName, uint32& OutTypeBitWidth, bool bBaseTypeOnly)
	{
		// Generate prefix for base type
		if (TypeSpecifier.type_flags & SPV_REFLECT_TYPE_FLAG_BOOL)
		{
			OutTypeName += TEXT('b');
			OutTypeBitWidth = 8;
		}
		else if (TypeSpecifier.type_flags & SPV_REFLECT_TYPE_FLAG_INT)
		{
			if (TypeSpecifier.traits.numeric.scalar.signedness)
			{
				OutTypeName += TEXT('i');
			}
			else
			{
				OutTypeName += TEXT('u');
			}
			OutTypeBitWidth = 32;
		}
		else if (TypeSpecifier.type_flags & SPV_REFLECT_TYPE_FLAG_FLOAT)
		{
			if (TypeSpecifier.traits.numeric.scalar.width == 16)
			{
				OutTypeName += TEXT('h');
				OutTypeBitWidth = 16;
			}
			else
			{
				OutTypeName += TEXT('f');
				OutTypeBitWidth = 32;
			}
		}

		if (!bBaseTypeOnly)
		{
			// Generate number for vector size
			const SpvReflectTypeFlags SpvScalarTypeFlags = (SPV_REFLECT_TYPE_FLAG_BOOL | SPV_REFLECT_TYPE_FLAG_INT | SPV_REFLECT_TYPE_FLAG_FLOAT);
			if (TypeSpecifier.type_flags & SPV_REFLECT_TYPE_FLAG_VECTOR)
			{
				static const TCHAR* VectorDims = TEXT("1234");
				const uint32 VectorSize = TypeSpecifier.traits.numeric.vector.component_count;
				check(VectorSize >= 1 && VectorSize <= 4);
				OutTypeName += VectorDims[VectorSize - 1];
			}
			else if (TypeSpecifier.type_flags & SPV_REFLECT_TYPE_FLAG_MATRIX)
			{
				//TODO
			}
			else if ((TypeSpecifier.type_flags & SpvScalarTypeFlags) != 0)
			{
				OutTypeName += TEXT('1'); // add single scalar component
			}
		}
	}

	static FString ConvertMetaDataTypeSpecifier(const SpvReflectTypeDescription& TypeSpecifier, uint32* OutTypeBitWidth = nullptr, bool bBaseTypeOnly = false)
	{
		FString TypeName;
		uint32 TypeBitWidth = sizeof(float) * 8;
		ConvertMetaDataTypeSpecifierPrimary(TypeSpecifier, TypeName, TypeBitWidth, bBaseTypeOnly);
		if (OutTypeBitWidth)
		{
			*OutTypeBitWidth = TypeBitWidth;
		}
		return TypeName;
	}

	static FString ConvertAttributeToMetaDataSemantic(const ANSICHAR* AttributeName, const SpvBuiltIn BuiltIn, bool bIsInput)
	{
		if (const TCHAR* BuiltInName = SpirvBuiltinToString(BuiltIn))
		{
			return FString(BuiltInName);
		}
		else
		{
			check(AttributeName != nullptr && *AttributeName != '\0');
			FString InSemantic = ANSI_TO_TCHAR(AttributeName);
			FString OutSemantic = (bIsInput ? TEXT("in_") : TEXT("out_"));

			if (InSemantic.StartsWith(TEXT("SV_")))
			{
				OutSemantic += InSemantic.Right(InSemantic.Len() - 3);
			}
			else
			{
				OutSemantic += InSemantic;
			}

			return OutSemantic;
		}
	}

	// Flattens the array dimensions of the interface variable (aka shader attribute), e.g. from float4[2][3] -> float4[6]
	static uint32 FlattenAttributeArrayDimension(const SpvReflectInterfaceVariable& Attribute, uint32 FirstArrayDim = 0)
	{
		uint32 FlattenedArrayDim = 1;
		for (uint32 ArrayDimIndex = FirstArrayDim; ArrayDimIndex < Attribute.array.dims_count; ++ArrayDimIndex)
		{
			FlattenedArrayDim *= Attribute.array.dims[ArrayDimIndex];
		}
		return FlattenedArrayDim;
	}

	// Returns the string position where the index in the specified HLSL semantic beings, e.g. "SV_Target2" -> 9, "SV_Target" -> INDEX_NONE
	static int32 FindIndexInHlslSemantic(const FString& Semantic)
	{
		int32 Index = Semantic.Len();
		if (Index > 0 && FChar::IsDigit(Semantic[Index - 1]))
		{
			while (Index > 0 && FChar::IsDigit(Semantic[Index - 1]))
			{
				--Index;
			}
			return Index;
		}
		return INDEX_NONE;
	}

	// private
	void FHlslccHeaderWriter::WriteIOAttribute(FString& OutMetaData, const TCHAR* AttributeName, const TCHAR* TypeSpecifier, int32 Location, bool bLocationPrefix, bool bLocationSuffix)
	{
		MetaDataPrintf(OutMetaData, TEXT("%s"), TypeSpecifier);
		if (bLocationPrefix)
		{
			OutMetaData += FString::Printf(TEXT(";%d:"), Location);
		}
		else
		{
			OutMetaData += TEXT(":");
		}
		if (bLocationSuffix)
		{
			OutMetaData += FString::Printf(TEXT("%s%d"), AttributeName, Location);
		}
		else
		{
			OutMetaData += AttributeName;
		}
	}

	// private
	void FHlslccHeaderWriter::WriteIOAttribute(FString& OutMetaData, const SpvReflectInterfaceVariable& Attribute, bool bIsInput)
	{
		// Ignore interface variables that are only generated for intermediate results
		if (CrossCompiler::FShaderConductorContext::IsIntermediateSpirvOutputVariable(Attribute.name))
		{
			return;
		}

		const FString TypeSpecifier = ConvertMetaDataTypeSpecifier(*Attribute.type_description);
		const FString Semantic = ConvertAttributeToMetaDataSemantic(Attribute.semantic, Attribute.built_in, bIsInput);

		if (Attribute.array.dims_count > 0)
		{
			if (Attribute.location == -1)
			{
				// Flatten array dimensions, e.g. from float4[3][2] -> float4[6]
				const uint32 FlattenedArrayDim = FlattenAttributeArrayDimension(Attribute);

				// Emit one output slot for each array element, e.g. "out float4 OutColor[2] : SV_Target0" occupies output slot SV_Target0 and SV_Target1.
				for (uint32 FlattenedArrayIndex = 0; FlattenedArrayIndex < FlattenedArrayDim; ++FlattenedArrayIndex)
				{
					// If there is no binding slot, emit output as system value array such as "gl_SampleMask[]"
					const uint32 BindingSlot = Attribute.location;
					MetaDataPrintf(
						OutMetaData,
						TEXT("%s;%d:%s[%d]"),
						*TypeSpecifier, // type specifier
						BindingSlot,
						*Semantic,
						FlattenedArrayIndex
					);
				}
			}
			else if (!bIsInput)
			{
				//NOTE: For some reason, the meta data for output slot arrays must be entirely flattened, including the outer most array dimension
				// Flatten array dimensions, e.g. from float4[3][2] -> float4[6]
				const uint32 FlattenedArrayDim = FlattenAttributeArrayDimension(Attribute);

				// Emit one output slot for each array element, e.g. "out float4 OutColor[2] : SV_Target0" occupies output slot SV_Target0 and SV_Target1.
				for (uint32 FlattenedArrayIndex = 0; FlattenedArrayIndex < FlattenedArrayDim; ++FlattenedArrayIndex)
				{
					const uint32 BindingSlot = Attribute.location + FlattenedArrayIndex;
					MetaDataPrintf(
						OutMetaData,
						TEXT("%s;%d:%s"),
						*TypeSpecifier, // Type specifier
						BindingSlot,
						*Semantic
					);
				}
			}
			else if (Attribute.array.dims_count >= 2)
			{
				// Flatten array dimensions, e.g. from float4[3][2] -> float4[6]
				const uint32 FlattenedArrayDim = FlattenAttributeArrayDimension(Attribute, 1);

				// Emit one output slot for each array element, e.g. "out float4 OutColor[2] : SV_Target0" occupies output slot SV_Target0 and SV_Target1.
				for (uint32 FlattenedArrayIndex = 0; FlattenedArrayIndex < FlattenedArrayDim; ++FlattenedArrayIndex)
				{
					const uint32 BindingSlot = Attribute.location + FlattenedArrayIndex;
					MetaDataPrintf(
						OutMetaData,
						TEXT("%s[%d];%d:%s"),
						*TypeSpecifier, // Type specifier
						Attribute.array.dims[0], // Outer most array dimension
						BindingSlot,
						*Semantic
					);
				}
			}
			else
			{
				const uint32 BindingSlot = Attribute.location;
				MetaDataPrintf(
					OutMetaData,
					TEXT("%s[%d];%d:%s"),
					*TypeSpecifier, // Type specifier
					Attribute.array.dims[0], // Outer most array dimension
					BindingSlot,
					*Semantic
				);
			}
		}
		else
		{
			MetaDataPrintf(
				OutMetaData,
				TEXT("%s;%d:%s"),
				*TypeSpecifier, // type specifier
				Attribute.location,
				*Semantic
			);
		}
	}

	void FHlslccHeaderWriter::WriteUniformBlock(const TCHAR* ResourceName, uint32 BindingIndex)
	{
		MetaDataPrintf(Strings.UniformBlocks, TEXT("%s(%u)"), ResourceName, BindingIndex);
	}

	EPackedTypeName FHlslccHeaderWriter::EncodePackedGlobalType(const SpvReflectTypeDescription& TypeDescription, bool bHalfPrecision)
	{
		const SpvReflectTypeFlags ScalarTypeFlagsBitmask =
		(
			SPV_REFLECT_TYPE_FLAG_VOID	|
			SPV_REFLECT_TYPE_FLAG_BOOL	|
			SPV_REFLECT_TYPE_FLAG_INT	|
			SPV_REFLECT_TYPE_FLAG_FLOAT
		);
		const SpvReflectTypeFlags TypeFlags = TypeDescription.type_flags;
		const SpvReflectTypeFlags MaskedType = TypeFlags & ScalarTypeFlagsBitmask;

		switch (MaskedType)
		{
		case SPV_REFLECT_TYPE_FLAG_BOOL:
		case SPV_REFLECT_TYPE_FLAG_INT:
			return (TypeDescription.traits.numeric.scalar.signedness ? EPackedTypeName::Int : EPackedTypeName::Uint);
		case SPV_REFLECT_TYPE_FLAG_FLOAT:
			if (bHalfPrecision)
			{
				return EPackedTypeName::MediumP;
			}
			else
			{
				return EPackedTypeName::HighP;
			}
			
		default:
			checkf(false, TEXT("unsupported component type %d"), MaskedType);
			return EPackedTypeName::LowP;
		}
	}

	void FHlslccHeaderWriter::WritePackedGlobal(const TCHAR* ResourceName, EPackedTypeName PackedType, uint32 ByteOffset, uint32 ByteSize)
	{
		// PackedType must be one of 'h','m','l','i','u'.
		checkf(ByteOffset % 4 == 0, TEXT("field offset of \"%s\" in @PackedGlobals shader meta data must be a multiple of 4, but got %u"), ResourceName, ByteOffset);
		checkf(ByteSize % 4 == 0, TEXT("field size of \"%s\" in @PackedGlobals shader meta data must be a multiple of 4, but got %u"), ResourceName, ByteSize);
		MetaDataPrintf(Strings.PackedGlobals, TEXT("%s(%c:%u,%u)"), ResourceName, TCHAR(PackedType), ByteOffset / 4, ByteSize / 4);
	}

	void FHlslccHeaderWriter::WritePackedGlobal(const SpvReflectBlockVariable& Variable)
	{
		const EPackedTypeName PackedType = EncodePackedGlobalType(*(Variable.type_description));
		WritePackedGlobal(ANSI_TO_TCHAR(Variable.name), PackedType, Variable.absolute_offset, Variable.size);// padded_size);
	}

	void FHlslccHeaderWriter::WritePackedUB(uint32 BindingIndex)
	{
		WritePackedUB(TEXT("Globals"), BindingIndex);
	}

	void FHlslccHeaderWriter::WritePackedUBField(const TCHAR* ResourceName, uint32 ByteOffset, uint32 ByteSize)
	{
		WritePackedUBField(TEXT("Globals"), ResourceName, ByteOffset, ByteSize);
	}

	void FHlslccHeaderWriter::WritePackedUB(const FString& UBName, uint32 BindingIndex)
	{
		checkf(Strings.PackedUBs.Find(UBName) == nullptr, TEXT("attempting to add a UB that has already been added"));
		MetaDataPrintf(Strings.PackedUBs.Add(UBName), TEXT("%s(%u): "), *UBName, BindingIndex);
	}

	void FHlslccHeaderWriter::WritePackedUBField(const FString& UBName, const TCHAR* ResourceName, uint32 ByteOffset, uint32 ByteSize)
	{
		checkf(Strings.PackedUBs.Find(UBName), TEXT("cannot append field without @PackedUB attribute in shader meta data, %s"), *UBName);
		checkf(ByteOffset % 4 == 0, TEXT("field offset in @PackedUB shader meta data must be a multiple of 4, but got %u"), ByteOffset);
		checkf(ByteSize % 4 == 0, TEXT("field size in @PackedUB shader meta data must be a multiple of 4, but got %u"), ByteSize);
		MetaDataPrintf(Strings.PackedUBFields.FindOrAdd(UBName), TEXT("%s(%u,%u)"), ResourceName, ByteOffset / 4, ByteSize / 4);
	}

	void FHlslccHeaderWriter::WritePackedUBCopy(uint32 SourceCB, uint32 SourceOffset, uint32 DestCBIndex, uint32 DestCBPrecision, uint32 DestOffset, uint32 Size, bool bGroupFlattenedUBs)
	{
		if (bGroupFlattenedUBs)
		{
			MetaDataPrintf(Strings.PackedUBCopies, TEXT("%u:%u-%u:%c:%u:%u"), SourceCB, SourceOffset, DestCBIndex, DestCBPrecision, DestOffset, Size);
		}
		else
		{
			check(DestCBIndex == 0);
			MetaDataPrintf(Strings.PackedUBCopies, TEXT("%u:%u-%c:%u:%u"), SourceCB, SourceOffset, DestCBPrecision, DestOffset, Size);
		}
	}

	void FHlslccHeaderWriter::WritePackedUBGlobalCopy(uint32 SourceCB, uint32 SourceOffset, uint32 DestCBIndex, uint32 DestCBPrecision, uint32 DestOffset, uint32 Size, bool bGroupFlattenedUBs)
	{
		if (bGroupFlattenedUBs)
		{
			MetaDataPrintf(Strings.PackedUBGlobalCopies, TEXT("%u:%u-%u:%c:%u:%u"), SourceCB, SourceOffset, DestCBIndex, DestCBPrecision, DestOffset, Size);
		}
		else
		{
			check(DestCBIndex == 0);
			MetaDataPrintf(Strings.PackedUBGlobalCopies, TEXT("%u:%u-%c:%u:%u"), SourceCB, SourceOffset, DestCBPrecision, DestOffset, Size);
		}
	}

	void FHlslccHeaderWriter::WriteSRV(const TCHAR* ResourceName, uint32 BindingIndex, uint32 Count)
	{
		MetaDataPrintf(Strings.SRVs, TEXT("%s(%u:%u)"), ResourceName, BindingIndex, Count);
	}

	void FHlslccHeaderWriter::WriteSRV(const TCHAR* ResourceName, uint32 BindingIndex, uint32 Count, const TArray<FString>& AssociatedResourceNames)
	{
		MetaDataPrintf(Strings.SRVs, TEXT("%s(%u:%u"), ResourceName, BindingIndex, Count);

		if (!AssociatedResourceNames.IsEmpty())
		{
			Strings.SRVs += TEXT("[");
			for (int32 ArrayIndex = 0; ArrayIndex < AssociatedResourceNames.Num(); ++ArrayIndex)
			{
				if (ArrayIndex > 0)
				{
					Strings.SRVs += TEXT(",");
				}
				Strings.SRVs += AssociatedResourceNames[ArrayIndex];
			}
			Strings.SRVs += TEXT("]");
		}

		Strings.SRVs += TEXT(")");
	}

	void FHlslccHeaderWriter::WriteUAV(const TCHAR* ResourceName, uint32 BindingIndex, uint32 Count)
	{
		MetaDataPrintf(Strings.UAVs, TEXT("%s(%u:%u)"), ResourceName, BindingIndex, Count);
	}

	void FHlslccHeaderWriter::WriteSamplerState(const TCHAR* ResourceName, uint32 BindingIndex)
	{
		MetaDataPrintf(Strings.SamplerStates, TEXT("%u:%s"), BindingIndex, ResourceName);
	}

	void FHlslccHeaderWriter::WriteNumThreads(uint32 NumThreadsX, uint32 NumThreadsY, uint32 NumThreadsZ)
	{
		MetaDataPrintf(Strings.NumThreads, TEXT("%u, %u, %u"), NumThreadsX, NumThreadsY, NumThreadsZ);
	}

	void FHlslccHeaderWriter::WriteAccelerationStructures(const TCHAR* ResourceName, uint32 BindingIndex)
	{
		MetaDataPrintf(Strings.AccelerationStructures, TEXT("%u:%s"), BindingIndex, ResourceName);
	}

	void FHlslccHeaderWriter::WriteSideTable(const TCHAR* ResourceName, uint32 SideTableIndex)
	{
		MetaDataPrintf(Strings.SideTable, TEXT("%s(%d)"), ResourceName, SideTableIndex);
	}

	void FHlslccHeaderWriter::WriteArgumentBuffers(uint32 BindingIndex, const TArray<uint32>& ResourceIndices)
	{
		MetaDataPrintf(Strings.ArgumentBuffers, TEXT("%d["), BindingIndex);

		for (int32 ArrayIndex = 0; ArrayIndex < ResourceIndices.Num(); ++ArrayIndex)
		{
			if (ArrayIndex > 0)
			{
				Strings.ArgumentBuffers += TEXT(",");
			}
			Strings.ArgumentBuffers += FString::Printf(TEXT("%u"), ResourceIndices[ArrayIndex]);
		}

		Strings.ArgumentBuffers += TEXT("]");
	}

	FString FHlslccHeaderWriter::ToString() const
	{
		FString MetaData;

		if (!Strings.SourceInfo.IsEmpty())
		{
			MetaData += FString::Printf(TEXT("// ! %s\n"), *Strings.SourceInfo);
		}

		if (!Strings.CompilerInfo.IsEmpty())
		{
			MetaData += FString::Printf(TEXT("// Compiled by %s\n"), *Strings.CompilerInfo);
		}

		auto PrintAttributes = [&MetaData](const TCHAR* Name, const FString& Value)
		{
			if (!Value.IsEmpty())
			{
				MetaData += FString::Printf(TEXT("// @%s: %s\n"), Name, *Value);
			}
		};

		PrintAttributes(TEXT("Inputs"), Strings.InputAttributes);
		PrintAttributes(TEXT("Outputs"), Strings.OutputAttributes);
		PrintAttributes(TEXT("UniformBlocks"), Strings.UniformBlocks);

		PrintAttributes(TEXT("PackedGlobals"), Strings.PackedGlobals);

		for (TPair<FString, FString> UBName : Strings.PackedUBs)
		{
			PrintAttributes(TEXT("PackedUB"), UBName.Value + Strings.PackedUBFields[UBName.Key]);
		}

		PrintAttributes(TEXT("PackedUBCopies"), Strings.PackedUBCopies);
		PrintAttributes(TEXT("PackedUBGlobalCopies"), Strings.PackedUBGlobalCopies);
		PrintAttributes(TEXT("Samplers"), Strings.SRVs); // Was called "Samplers" in HLSLcc but serves as SRVs
		PrintAttributes(TEXT("UAVs"), Strings.UAVs);
		PrintAttributes(TEXT("SamplerStates"), Strings.SamplerStates);
		PrintAttributes(TEXT("NumThreads"), Strings.NumThreads);
		PrintAttributes(TEXT("AccelerationStructures"), Strings.AccelerationStructures);
		PrintAttributes(TEXT("SideTable"), Strings.SideTable);
		PrintAttributes(TEXT("ArgumentBuffers"), Strings.ArgumentBuffers);

		return MetaData;
	}

} // namespace CrossCompiler
