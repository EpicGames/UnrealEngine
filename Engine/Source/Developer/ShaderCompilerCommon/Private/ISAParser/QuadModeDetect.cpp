// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISAParser/ISAParser.h"
#include "RDNA1_ISA.inl"
#include "RDNA2_ISA.inl"

struct FDebugParserRDNA1
{
	uint32 AdjustAdvance(EInstructionType InstructionType, const uint32* InstPtr)
	{
		uint32 AdvanceAmount = 0u;

		if (InstructionType == EInstructionType::VOP2)
		{
			const FInstVOP2& Inst = *reinterpret_cast<const FInstVOP2*>(InstPtr);
			const RDNA1::EVOP2Ops Op = (const RDNA1::EVOP2Ops)Inst.OP;

			// These instructions always have a literal constant that follows
			if (RDNA1::HasTrailingLiteral(Op))
			{
				AdvanceAmount++;
			}
		}

		return AdvanceAmount;
	}

	bool ParseSOP1(const FInstSOP1& Inst)
	{
		RDNA1::PrintSOP1(Inst);
		return true;
	}

	bool ParseSOP2(const FInstSOP2& Inst)
	{
		RDNA1::PrintSOP2(Inst);
		return true;
	}

	bool ParseSOPP(const FInstSOPP& Inst)
	{
		RDNA1::PrintSOPP(Inst);

		const RDNA1::ESOPPOps Op = (const RDNA1::ESOPPOps)Inst.OP;
		return (Op != RDNA1::ESOPPOps::s_endpgm);
	}

	bool ParseSOPK(const FInstSOPK& Inst)
	{
		RDNA1::PrintSOPK(Inst);
		return true;
	}

	bool ParseSOPC(const FInstSOPC& Inst)
	{
		RDNA1::PrintSOPC(Inst);
		return true;
	}

	bool ParseSMEM(const FInstSMEM& Inst)
	{
		RDNA1::PrintSMEM(Inst);
		return true;
	}

	bool ParseVOP1(const FInstVOP1& Inst)
	{
		RDNA1::PrintVOP1(Inst);
		return true;
	}

	bool ParseVOP1_DPP16(const FInstVOP1& Inst, const FDPP16& DPP16)
	{
		RDNA1::PrintVOP1(Inst);
		return true;
	}
	
	bool ParseVOP2(const FInstVOP2& Inst)
	{
		RDNA1::PrintVOP2(Inst);
		return true;
	}

	bool ParseVOP2_DPP16(const FInstVOP2& Inst, const FDPP16& DPP16)
	{
		RDNA1::PrintVOP2(Inst);
		if (DPP16.DPPCTRL <= 0x0FF)
		{
			// DPP_QUAD_PERM
		}
		return true;
	}

	bool ParseVOP3(const FInstVOP3& Inst)
	{
		if (RDNA1::IsVOP3BEncoding((RDNA1::EVOP3ABOps)Inst.Encoding.VOP3B.OP))
		{
			RDNA1::PrintVOP3B(Inst.Encoding.VOP3B);
		}
		else
		{
			RDNA1::PrintVOP3A(Inst.Encoding.VOP3A);
		}

		return true;
	}

	bool ParseVOPC(const FInstVOPC& Inst)
	{
		RDNA1::PrintVOPC(Inst);
		return true;
	}

	bool ParseVOPC_DPP16(const FInstVOPC& Inst, const FDPP16& DPP16)
	{
		RDNA1::PrintVOPC(Inst);
		return true;
	}

	bool ParseVOP3P(const FInstVOP3P& Inst)
	{
		RDNA1::PrintVOP3P(Inst);
		return true;
	}

	bool ParseVINTERP(const FInstVINTERP& Inst)
	{
		RDNA1::PrintVINTERP(Inst);
		return true;
	}

	bool ParseLDSGDS(const FInstLDSGDS& Inst)
	{
		RDNA1::PrintLDSGDS(Inst);
		return true;
	}

	bool ParseMUBUF(const FInstMUBUF& Inst)
	{
		RDNA1::PrintMUBUF(Inst);
		return true;
	}

	bool ParseMTBUF(const FInstMTBUF& Inst)
	{
		RDNA1::PrintMTBUF(Inst);
		return true;
	}

	bool ParseEXPORT(const FInstEXPORT& Inst)
	{
		RDNA1::PrintEXPORT(Inst);
		return true;
	}

	bool ParseFSG(const FInstFSG& Inst)
	{
		RDNA1::PrintFSG(Inst);
		return true;
	}

	bool ParseMIMG(const FInstMIMG& Inst)
	{
		RDNA1::PrintMIMG(Inst);
		return true;
	}
};

struct FDebugParserRDNA2
{
	uint32 AdjustAdvance(EInstructionType InstructionType, const uint32* InstPtr)
	{
		uint32 AdvanceAmount = 0u;

		if (InstructionType == EInstructionType::VOP2)
		{
			const FInstVOP2& Inst = *reinterpret_cast<const FInstVOP2*>(InstPtr);
			const RDNA2::EVOP2Ops Op = (const RDNA2::EVOP2Ops)Inst.OP;

			// These instructions always have a literal constant that follows
			if (RDNA2::HasTrailingLiteral(Op))
			{
				AdvanceAmount++;
			}
		}

		return AdvanceAmount;
	}

	bool ParseSOP1(const FInstSOP1& Inst)
	{
		RDNA2::PrintSOP1(Inst);
		return true;
	}

	bool ParseSOP2(const FInstSOP2& Inst)
	{
		RDNA2::PrintSOP2(Inst);
		return true;
	}

	bool ParseSOPP(const FInstSOPP& Inst)
	{
		RDNA2::PrintSOPP(Inst);

		const RDNA2::ESOPPOps Op = (const RDNA2::ESOPPOps)Inst.OP;
		return (Op != RDNA2::ESOPPOps::s_endpgm);
	}

	bool ParseSOPK(const FInstSOPK& Inst)
	{
		RDNA2::PrintSOPK(Inst);
		return true;
	}

	bool ParseSOPC(const FInstSOPC& Inst)
	{
		RDNA2::PrintSOPC(Inst);
		return true;
	}

	bool ParseSMEM(const FInstSMEM& Inst)
	{
		RDNA2::PrintSMEM(Inst);
		return true;
	}

	bool ParseVOP1(const FInstVOP1& Inst)
	{
		RDNA2::PrintVOP1(Inst);
		return true;
	}

	bool ParseVOP1_DPP16(const FInstVOP1& Inst, const FDPP16& DPP16)
	{
		RDNA2::PrintVOP1(Inst);
		return true;
	}
	
	bool ParseVOP2(const FInstVOP2& Inst)
	{
		RDNA2::PrintVOP2(Inst);
		return true;
	}

	bool ParseVOP2_DPP16(const FInstVOP2& Inst, const FDPP16& DPP16)
	{
		RDNA2::PrintVOP2(Inst);
		if (DPP16.DPPCTRL <= 0x0FF)
		{
			// DPP_QUAD_PERM
		}
		return true;
	}

	bool ParseVOP3(const FInstVOP3& Inst)
	{
		if (RDNA2::IsVOP3BEncoding((RDNA2::EVOP3ABOps)Inst.Encoding.VOP3B.OP))
		{
			RDNA2::PrintVOP3B(Inst.Encoding.VOP3B);
		}
		else
		{
			RDNA2::PrintVOP3A(Inst.Encoding.VOP3A);
		}

		return true;
	}

	bool ParseVOPC(const FInstVOPC& Inst)
	{
		RDNA2::PrintVOPC(Inst);
		return true;
	}

	bool ParseVOPC_DPP16(const FInstVOPC& Inst, const FDPP16& DPP16)
	{
		RDNA2::PrintVOPC(Inst);
		return true;
	}

	bool ParseVOP3P(const FInstVOP3P& Inst)
	{
		RDNA2::PrintVOP3P(Inst);
		return true;
	}

	bool ParseVINTERP(const FInstVINTERP& Inst)
	{
		RDNA2::PrintVINTERP(Inst);
		return true;
	}

	bool ParseLDSGDS(const FInstLDSGDS& Inst)
	{
		RDNA2::PrintLDSGDS(Inst);
		return true;
	}

	bool ParseMUBUF(const FInstMUBUF& Inst)
	{
		RDNA2::PrintMUBUF(Inst);
		return true;
	}

	bool ParseMTBUF(const FInstMTBUF& Inst)
	{
		RDNA2::PrintMTBUF(Inst);
		return true;
	}

	bool ParseEXPORT(const FInstEXPORT& Inst)
	{
		RDNA2::PrintEXPORT(Inst);
		return true;
	}

	bool ParseFSG(const FInstFSG& Inst)
	{
		RDNA2::PrintFSG(Inst);
		return true;
	}

	bool ParseMIMG(const FInstMIMG& Inst)
	{
		RDNA2::PrintMIMG(Inst);
		return true;
	}
};

struct FQuadModeParserRDNA1
{
	bool bQuadMode = false;
	bool bDebug = false;

	uint32 AdjustAdvance(EInstructionType InstructionType, const uint32* InstPtr)
	{
		uint32 AdvanceAmount = 0u;

		if (InstructionType == EInstructionType::VOP2)
		{
			const FInstVOP2& Inst = *reinterpret_cast<const FInstVOP2*>(InstPtr);
			const RDNA1::EVOP2Ops Op = (const RDNA1::EVOP2Ops)Inst.OP;

			// These instructions always have a literal constant that follows
			if (RDNA1::HasTrailingLiteral(Op))
			{
				AdvanceAmount++;
			}
		}

		return AdvanceAmount;
	}

	inline bool KeepParsing() const
	{
		// If we detected the need for quad mode, and are not debugging, stop parsing early
		return !bQuadMode || bDebug;
	}

	bool IsDPPQuadPerm(const FDPP16& DPP16)
	{
		return (DPP16.DPPCTRL <= 0x0FF); // DPP_QUAD_PERM
	}

	bool ParseSOPP(const FInstSOPP& Inst)
	{
		const RDNA1::ESOPPOps Op = (const RDNA1::ESOPPOps)Inst.OP;
		return (Op != RDNA1::ESOPPOps::s_endpgm) && KeepParsing();
	}

	bool ParseSOP1(const FInstSOP1& Inst)		{ return KeepParsing(); }
	bool ParseSOP2(const FInstSOP2& Inst)		{ return KeepParsing(); }
	bool ParseSOPK(const FInstSOPK& Inst)		{ return KeepParsing(); }
	bool ParseSOPC(const FInstSOPC& Inst)		{ return KeepParsing(); }
	bool ParseSMEM(const FInstSMEM& Inst)		{ return KeepParsing(); }
	bool ParseVOP1(const FInstVOP1& Inst)		{ return KeepParsing(); }
	bool ParseVOP2(const FInstVOP2& Inst)		{ return KeepParsing(); }
	bool ParseVOP3(const FInstVOP3& Inst)		{ return KeepParsing(); }
	bool ParseVOPC(const FInstVOPC& Inst)		{ return KeepParsing(); }
	bool ParseVOP3P(const FInstVOP3P& Inst)		{ return KeepParsing(); }
	bool ParseVINTERP(const FInstVINTERP& Inst)	{ return KeepParsing(); }
	bool ParseLDSGDS(const FInstLDSGDS& Inst)	{ return KeepParsing(); }
	bool ParseMUBUF(const FInstMUBUF& Inst)		{ return KeepParsing(); }
	bool ParseMTBUF(const FInstMTBUF& Inst)		{ return KeepParsing(); }
	bool ParseEXPORT(const FInstEXPORT& Inst)	{ return KeepParsing(); }
	bool ParseFSG(const FInstFSG& Inst)			{ return KeepParsing(); }

	bool ParseVOP1_DPP16(const FInstVOP1& Inst, const FDPP16& DPP16)
	{
		if (IsDPPQuadPerm(DPP16))
		{
			bQuadMode = true;
			if (bDebug)
			{
				RDNA1::PrintVOP1(Inst);
			}
		}

		return KeepParsing();
	}

	bool ParseVOP2_DPP16(const FInstVOP2& Inst, const FDPP16& DPP16)
	{
		if (IsDPPQuadPerm(DPP16))
		{
			bQuadMode = true;
			if (bDebug)
			{
				RDNA1::PrintVOP2(Inst);
			}
		}

		return KeepParsing();
	}

	bool ParseVOPC_DPP16(const FInstVOPC& Inst, const FDPP16& DPP16)
	{
		if (IsDPPQuadPerm(DPP16))
		{
			bQuadMode = true;
			if (bDebug)
			{
				RDNA1::PrintVOPC(Inst);
			}
		}

		return KeepParsing();
	}

	bool ParseMIMG(const FInstMIMG& Inst)
	{
		const RDNA1::EMIMGOps Op = (const RDNA1::EMIMGOps)Inst.OP;

		switch (Op)
		{
			case RDNA1::EMIMGOps::image_sample:
			case RDNA1::EMIMGOps::image_sample_cl:
			case RDNA1::EMIMGOps::image_sample_b:
			case RDNA1::EMIMGOps::image_sample_b_cl:
			case RDNA1::EMIMGOps::image_sample_c:
			case RDNA1::EMIMGOps::image_sample_c_cl:
			case RDNA1::EMIMGOps::image_sample_c_b:
			case RDNA1::EMIMGOps::image_sample_c_b_cl:
			case RDNA1::EMIMGOps::image_sample_o:
			case RDNA1::EMIMGOps::image_sample_cl_o:
			case RDNA1::EMIMGOps::image_sample_b_o:
			case RDNA1::EMIMGOps::image_sample_b_cl_o:
			case RDNA1::EMIMGOps::image_sample_c_o:
			case RDNA1::EMIMGOps::image_sample_c_cl_o:
			case RDNA1::EMIMGOps::image_sample_c_b_o:
			case RDNA1::EMIMGOps::image_sample_c_b_cl_o:
			case RDNA1::EMIMGOps::image_gather4:
			case RDNA1::EMIMGOps::image_gather4_cl:
			case RDNA1::EMIMGOps::image_gather4_b:
			case RDNA1::EMIMGOps::image_gather4_b_cl:
			case RDNA1::EMIMGOps::image_gather4_c:
			case RDNA1::EMIMGOps::image_gather4_c_cl:
			case RDNA1::EMIMGOps::image_gather4_c_b:
			case RDNA1::EMIMGOps::image_gather4_c_b_cl:
			case RDNA1::EMIMGOps::image_gather4_o:
			case RDNA1::EMIMGOps::image_gather4_cl_o:
			case RDNA1::EMIMGOps::image_gather4_b_o:
			case RDNA1::EMIMGOps::image_gather4_b_cl_o:
			case RDNA1::EMIMGOps::image_gather4_c_o:
			case RDNA1::EMIMGOps::image_gather4_c_cl_o:
			case RDNA1::EMIMGOps::image_gather4_c_b_o:
			case RDNA1::EMIMGOps::image_gather4_c_b_cl_o:
			case RDNA1::EMIMGOps::image_get_lod:
			case RDNA1::EMIMGOps::image_gather4h:
			{
				if (bDebug)
				{
					RDNA1::PrintMIMG(Inst);
				}
				bQuadMode = true;
				break;
			}
		default:
			break;
		}

		return KeepParsing();
	}
};

struct FQuadModeParserRDNA2
{
	bool bQuadMode = false;
	bool bDebug = false;

	uint32 AdjustAdvance(EInstructionType InstructionType, const uint32* InstPtr)
	{
		uint32 AdvanceAmount = 0u;

		if (InstructionType == EInstructionType::VOP2)
		{
			const FInstVOP2& Inst = *reinterpret_cast<const FInstVOP2*>(InstPtr);
			const RDNA2::EVOP2Ops Op = (const RDNA2::EVOP2Ops)Inst.OP;

			// These instructions always have a literal constant that follows
			if (RDNA2::HasTrailingLiteral(Op))
			{
				AdvanceAmount++;
			}
		}

		return AdvanceAmount;
	}

	inline bool KeepParsing() const
	{
		// If we detected the need for quad mode, and are not debugging, stop parsing early
		return !bQuadMode || bDebug;
	}

	bool IsDPPQuadPerm(const FDPP16& DPP16)
	{
		return (DPP16.DPPCTRL <= 0x0FF); // DPP_QUAD_PERM
	}

	bool ParseSOPP(const FInstSOPP& Inst)
	{
		const RDNA2::ESOPPOps Op = (const RDNA2::ESOPPOps)Inst.OP;
		return (Op != RDNA2::ESOPPOps::s_endpgm) && KeepParsing();
	}

	bool ParseSOP1(const FInstSOP1& Inst)		{ return KeepParsing(); }
	bool ParseSOP2(const FInstSOP2& Inst)		{ return KeepParsing(); }
	bool ParseSOPK(const FInstSOPK& Inst)		{ return KeepParsing(); }
	bool ParseSOPC(const FInstSOPC& Inst)		{ return KeepParsing(); }
	bool ParseSMEM(const FInstSMEM& Inst)		{ return KeepParsing(); }
	bool ParseVOP1(const FInstVOP1& Inst)		{ return KeepParsing(); }
	bool ParseVOP2(const FInstVOP2& Inst)		{ return KeepParsing(); }
	bool ParseVOP3(const FInstVOP3& Inst)		{ return KeepParsing(); }
	bool ParseVOPC(const FInstVOPC& Inst)		{ return KeepParsing(); }
	bool ParseVOP3P(const FInstVOP3P& Inst)		{ return KeepParsing(); }
	bool ParseVINTERP(const FInstVINTERP& Inst)	{ return KeepParsing(); }
	bool ParseLDSGDS(const FInstLDSGDS& Inst)	{ return KeepParsing(); }
	bool ParseMUBUF(const FInstMUBUF& Inst)		{ return KeepParsing(); }
	bool ParseMTBUF(const FInstMTBUF& Inst)		{ return KeepParsing(); }
	bool ParseEXPORT(const FInstEXPORT& Inst)	{ return KeepParsing(); }
	bool ParseFSG(const FInstFSG& Inst)			{ return KeepParsing(); }

	bool ParseVOP1_DPP16(const FInstVOP1& Inst, const FDPP16& DPP16)
	{
		if (IsDPPQuadPerm(DPP16))
		{
			bQuadMode = true;
			if (bDebug)
			{
				RDNA2::PrintVOP1(Inst);
			}
		}

		return KeepParsing();
	}

	bool ParseVOP2_DPP16(const FInstVOP2& Inst, const FDPP16& DPP16)
	{
		if (IsDPPQuadPerm(DPP16))
		{
			bQuadMode = true;
			if (bDebug)
			{
				RDNA2::PrintVOP2(Inst);
			}
		}

		return KeepParsing();
	}

	bool ParseVOPC_DPP16(const FInstVOPC& Inst, const FDPP16& DPP16)
	{
		if (IsDPPQuadPerm(DPP16))
		{
			bQuadMode = true;
			if (bDebug)
			{
				RDNA2::PrintVOPC(Inst);
			}
		}

		return KeepParsing();
	}

	bool ParseMIMG(const FInstMIMG& Inst)
	{
		const RDNA2::EMIMGOps Op = (const RDNA2::EMIMGOps)Inst.OP;

		switch (Op)
		{
			case RDNA2::EMIMGOps::image_sample:
			case RDNA2::EMIMGOps::image_sample_cl:
			case RDNA2::EMIMGOps::image_sample_b:
			case RDNA2::EMIMGOps::image_sample_b_cl:
			case RDNA2::EMIMGOps::image_sample_c:
			case RDNA2::EMIMGOps::image_sample_c_cl:
			case RDNA2::EMIMGOps::image_sample_c_b:
			case RDNA2::EMIMGOps::image_sample_c_b_cl:
			case RDNA2::EMIMGOps::image_sample_o:
			case RDNA2::EMIMGOps::image_sample_cl_o:
			case RDNA2::EMIMGOps::image_sample_b_o:
			case RDNA2::EMIMGOps::image_sample_b_cl_o:
			case RDNA2::EMIMGOps::image_sample_c_o:
			case RDNA2::EMIMGOps::image_sample_c_cl_o:
			case RDNA2::EMIMGOps::image_sample_c_b_o:
			case RDNA2::EMIMGOps::image_sample_c_b_cl_o:
			case RDNA2::EMIMGOps::image_gather4:
			case RDNA2::EMIMGOps::image_gather4_cl:
			case RDNA2::EMIMGOps::image_gather4_b:
			case RDNA2::EMIMGOps::image_gather4_b_cl:
			case RDNA2::EMIMGOps::image_gather4_c:
			case RDNA2::EMIMGOps::image_gather4_c_cl:
			case RDNA2::EMIMGOps::image_gather4_c_b:
			case RDNA2::EMIMGOps::image_gather4_c_b_cl:
			case RDNA2::EMIMGOps::image_gather4_o:
			case RDNA2::EMIMGOps::image_gather4_cl_o:
			case RDNA2::EMIMGOps::image_gather4_b_o:
			case RDNA2::EMIMGOps::image_gather4_b_cl_o:
			case RDNA2::EMIMGOps::image_gather4_c_o:
			case RDNA2::EMIMGOps::image_gather4_c_cl_o:
			case RDNA2::EMIMGOps::image_gather4_c_b_o:
			case RDNA2::EMIMGOps::image_gather4_c_b_cl_o:
			case RDNA2::EMIMGOps::image_get_lod:
			case RDNA2::EMIMGOps::image_gather4h:
			case RDNA2::EMIMGOps::image_gather4h_pck:
			case RDNA2::EMIMGOps::image_gather8h_pck:
			{
				if (bDebug)
				{
					RDNA2::PrintMIMG(Inst);
				}
				bQuadMode = true;
				break;
			}
		default:
			break;
		}

		return KeepParsing();
	}
};

template <class PARSER>
bool PerformParsing(PARSER& Parser, const uint32* ReadPtr, const uint32* EndPtr)
{
	bool bSuccess = true;

	bool bParsing = true;
	while (ReadPtr < EndPtr && bParsing)
	{
		uint32_t AdvanceAmount = uint32_t(EndPtr - ReadPtr);
		EInstructionType InstructionType = DecodeInstructionType(ReadPtr, AdvanceAmount);
		AdvanceAmount += Parser.AdjustAdvance(InstructionType, ReadPtr);

		if (InstructionType == EInstructionType::UNKNOWN)
		{
			//UE_LOGF(LogTemp, Error, "Failed ISA Parse - Unknown Instruction");
			bSuccess = false;
			break;
		}
		else if (ReadPtr + AdvanceAmount >= EndPtr)
		{
			//UE_LOGF(LogTemp, Error, "Failed ISA Parse - Overfetch");
			bSuccess = false;
			break;
		}
		else if (InstructionType == EInstructionType::VOP1)
		{
			const FInstVOP1& Inst = *reinterpret_cast<const FInstVOP1*>(ReadPtr);
			if (Inst.SRC0 == Operand_DPP16)
			{
				const FDPP16& DPP16 = *reinterpret_cast<const FDPP16*>(ReadPtr + 1);
				bParsing = Parser.ParseVOP1_DPP16(Inst, DPP16);
			}
			else
			{
				bParsing = Parser.ParseVOP1(Inst);
			}
		}
		else if (InstructionType == EInstructionType::VOP2)
		{
			const FInstVOP2& Inst = *reinterpret_cast<const FInstVOP2*>(ReadPtr);
			if (Inst.SRC0 == Operand_DPP16)
			{
				const FDPP16& DPP16 = *reinterpret_cast<const FDPP16*>(ReadPtr + GetAdvanceAmount(EInstructionType::VOP2));
				bParsing = Parser.ParseVOP2_DPP16(Inst, DPP16);
			}
			else
			{
				bParsing = Parser.ParseVOP2(Inst);
			}
		}
		else if (InstructionType == EInstructionType::VOPC)
		{
			const FInstVOPC& Inst = *reinterpret_cast<const FInstVOPC*>(ReadPtr);
			if (Inst.SRC0 == Operand_DPP16)
			{
				const FDPP16& DPP16 = *reinterpret_cast<const FDPP16*>(ReadPtr + 1);
				bParsing = Parser.ParseVOPC_DPP16(Inst, DPP16);
			}
			else
			{
				bParsing = Parser.ParseVOPC(Inst);
			}
		}
	#define PARSE_MICROCODE_FORMAT(FORMAT) \
		else if (InstructionType == EInstructionType::FORMAT) \
		{ \
			bParsing = Parser.Parse##FORMAT(*reinterpret_cast<const FInst##FORMAT*>(ReadPtr)); \
		}

		PARSE_MICROCODE_FORMAT(SOP1)
		PARSE_MICROCODE_FORMAT(SOP2)
		PARSE_MICROCODE_FORMAT(SOPP)
		PARSE_MICROCODE_FORMAT(SOPK)
		PARSE_MICROCODE_FORMAT(SOPC)
		PARSE_MICROCODE_FORMAT(SMEM)
		PARSE_MICROCODE_FORMAT(VOP3)
		PARSE_MICROCODE_FORMAT(VOP3P)
		PARSE_MICROCODE_FORMAT(VINTERP)
		PARSE_MICROCODE_FORMAT(LDSGDS)
		PARSE_MICROCODE_FORMAT(MUBUF)
		PARSE_MICROCODE_FORMAT(MTBUF)
		PARSE_MICROCODE_FORMAT(EXPORT)
		PARSE_MICROCODE_FORMAT(FSG)
		PARSE_MICROCODE_FORMAT(MIMG)

	#undef PARSE_MICROCODE_FORMAT

		ReadPtr += AdvanceAmount;
	}

	return bSuccess;
}

#define DEBUG_DUMP_FAILURES 0

bool ISAParser::HasDerivativeOps(bool& bHasDerivativeOps, const char* Code, uint32 CodeLength, EInstructionSet InstructionSet)
{
	const uint32* ReadPtr = reinterpret_cast<const uint32*>(Code);
	const uint32* EndPtr  = reinterpret_cast<const uint32*>(Code + CodeLength);

	// Safe default, any error and behavior is unchanged
	bHasDerivativeOps = true;

	if (InstructionSet == EInstructionSet::RDNA1)
	{
		FQuadModeParserRDNA1 Parser;
		if (PerformParsing(Parser, ReadPtr, EndPtr))
		{
			bHasDerivativeOps = Parser.bQuadMode;
		}
		else
		{
		#if DEBUG_DUMP_FAILURES
			FDebugParserRDNA1 PrintParser;
			PerformParsing(PrintParser, ReadPtr, EndPtr);
		#endif

			return false;
		}
	}
	else if (InstructionSet == EInstructionSet::RDNA2)
	{
		FQuadModeParserRDNA2 Parser;
		if (PerformParsing(Parser, ReadPtr, EndPtr))
		{
			bHasDerivativeOps = Parser.bQuadMode;
		}
		else
		{
		#if DEBUG_DUMP_FAILURES
			FDebugParserRDNA2 PrintParser;
			PerformParsing(PrintParser, ReadPtr, EndPtr);
		#endif

			return false;
		}
	}
	else
	{
		checkNoEntry();
		return false;
	}

	return true;
}
