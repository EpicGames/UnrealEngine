// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RDNA_Common.h"
#include "HAL/OutputDevices.h"

// https://www.amd.com/content/dam/amd/en/documents/radeon-tech-docs/instruction-set-architectures/rdna2-shader-instruction-set-architecture.pdf

namespace RDNA2
{

enum class ESOP1Ops : uint16
{
	s_mov_b32 = 3,
	s_mov_b64 = 4,
	s_cmov_b32 = 5,
	s_cmov_b64 = 6,
	s_not_b32 = 7,
	s_not_b64 = 8,
	s_wqm_b32 = 9,
	s_wqm_b64 = 10,
	s_brev_b32 = 11,
	s_brev_b64 = 12,
	s_bcnt0_i32_b32 = 13,
	s_bcnt0_i32_b64 = 14,
	s_bcnt1_i32_b32 = 15,
	s_bcnt1_i32_b64 = 16,
	s_ff0_i32_b32 = 17,
	s_ff0_i32_b64 = 18,
	s_ff1_i32_b32 = 19,
	s_ff1_i32_b64 = 20,
	s_flbit_i32_b32 = 21,
	s_flbit_i32_b64 = 22,
	s_flbit_i32 = 23,
	s_flbit_i32_i64 = 24,
	s_sext_i32_i8 = 25,
	s_sext_i32_i16 = 26,
	s_bitset0_b32 = 27,
	s_bitset0_b64 = 28,
	s_bitset1_b32 = 29,
	s_bitset1_b64 = 30,
	s_getpc_b64 = 31,
	s_setpc_b64 = 32,
	s_swappc_b64 = 33,
	s_rfe_b64 = 34,
	s_and_saveexec_b64 = 36,
	s_or_saveexec_b64 = 37,
	s_xor_saveexec_b64 = 38,
	s_andn2_saveexec_b64 = 39,
	s_orn2_saveexec_b64 = 40,
	s_nand_saveexec_b64 = 41,
	s_nor_saveexec_b64 = 42,
	s_xnor_saveexec_b64 = 43,
	s_quadmask_b32 = 44,
	s_quadmask_b64 = 45,
	s_movrels_b32 = 46,
	s_movrels_b64 = 47,
	s_movreld_b32 = 48,
	s_movreld_b64 = 49,
	s_abs_i32 = 52,
	s_andn1_saveexec_b64 = 55,
	s_orn1_saveexec_b64 = 56,
	s_andn1_wrexec_b64 = 57,
	s_andn2_wrexec_b64 = 58,
	s_bitreplicate_b64_b32 = 59,
	s_and_saveexec_b32 = 60,
	s_or_saveexec_b32 = 61,
	s_xor_saveexec_b32 = 62,
	s_andn2_saveexec_b32 = 63,
	s_orn2_saveexec_b32 = 64,
	s_nand_saveexec_b32 = 65,
	s_nor_saveexec_b32 = 66,
	s_xnor_saveexec_b32 = 67,
	s_andn1_saveexec_b32 = 68,
	s_orn1_saveexec_b32 = 69,
	s_and1_wrexec_b32 = 70,
	s_andn2_wrexec_b32 = 71,
	s_movrelsd_2_b32 = 73,
};

const char* ToString(ESOP1Ops Op)
{
#define OP_TO_STRING_CASE(x) case ESOP1Ops::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(s_mov_b32);
		OP_TO_STRING_CASE(s_mov_b64);
		OP_TO_STRING_CASE(s_cmov_b32);
		OP_TO_STRING_CASE(s_cmov_b64);
		OP_TO_STRING_CASE(s_not_b32);
		OP_TO_STRING_CASE(s_not_b64);
		OP_TO_STRING_CASE(s_wqm_b32);
		OP_TO_STRING_CASE(s_wqm_b64);
		OP_TO_STRING_CASE(s_brev_b32);
		OP_TO_STRING_CASE(s_brev_b64);
		OP_TO_STRING_CASE(s_bcnt0_i32_b32);
		OP_TO_STRING_CASE(s_bcnt0_i32_b64);
		OP_TO_STRING_CASE(s_bcnt1_i32_b32);
		OP_TO_STRING_CASE(s_bcnt1_i32_b64);
		OP_TO_STRING_CASE(s_ff0_i32_b32);
		OP_TO_STRING_CASE(s_ff0_i32_b64);
		OP_TO_STRING_CASE(s_ff1_i32_b32);
		OP_TO_STRING_CASE(s_ff1_i32_b64);
		OP_TO_STRING_CASE(s_flbit_i32_b32);
		OP_TO_STRING_CASE(s_flbit_i32_b64);
		OP_TO_STRING_CASE(s_flbit_i32);
		OP_TO_STRING_CASE(s_flbit_i32_i64);
		OP_TO_STRING_CASE(s_sext_i32_i8);
		OP_TO_STRING_CASE(s_sext_i32_i16);
		OP_TO_STRING_CASE(s_bitset0_b32);
		OP_TO_STRING_CASE(s_bitset0_b64);
		OP_TO_STRING_CASE(s_bitset1_b32);
		OP_TO_STRING_CASE(s_bitset1_b64);
		OP_TO_STRING_CASE(s_getpc_b64);
		OP_TO_STRING_CASE(s_setpc_b64);
		OP_TO_STRING_CASE(s_swappc_b64);
		OP_TO_STRING_CASE(s_rfe_b64);
		OP_TO_STRING_CASE(s_and_saveexec_b64);
		OP_TO_STRING_CASE(s_or_saveexec_b64);
		OP_TO_STRING_CASE(s_xor_saveexec_b64);
		OP_TO_STRING_CASE(s_andn2_saveexec_b64);
		OP_TO_STRING_CASE(s_orn2_saveexec_b64);
		OP_TO_STRING_CASE(s_nand_saveexec_b64);
		OP_TO_STRING_CASE(s_nor_saveexec_b64);
		OP_TO_STRING_CASE(s_xnor_saveexec_b64);
		OP_TO_STRING_CASE(s_quadmask_b32);
		OP_TO_STRING_CASE(s_quadmask_b64);
		OP_TO_STRING_CASE(s_movrels_b32);
		OP_TO_STRING_CASE(s_movrels_b64);
		OP_TO_STRING_CASE(s_movreld_b32);
		OP_TO_STRING_CASE(s_movreld_b64);
		OP_TO_STRING_CASE(s_abs_i32);
		OP_TO_STRING_CASE(s_andn1_saveexec_b64);
		OP_TO_STRING_CASE(s_orn1_saveexec_b64);
		OP_TO_STRING_CASE(s_andn1_wrexec_b64);
		OP_TO_STRING_CASE(s_andn2_wrexec_b64);
		OP_TO_STRING_CASE(s_bitreplicate_b64_b32);
		OP_TO_STRING_CASE(s_and_saveexec_b32);
		OP_TO_STRING_CASE(s_or_saveexec_b32);
		OP_TO_STRING_CASE(s_xor_saveexec_b32);
		OP_TO_STRING_CASE(s_andn2_saveexec_b32);
		OP_TO_STRING_CASE(s_orn2_saveexec_b32);
		OP_TO_STRING_CASE(s_nand_saveexec_b32);
		OP_TO_STRING_CASE(s_nor_saveexec_b32);
		OP_TO_STRING_CASE(s_xnor_saveexec_b32);
		OP_TO_STRING_CASE(s_andn1_saveexec_b32);
		OP_TO_STRING_CASE(s_orn1_saveexec_b32);
		OP_TO_STRING_CASE(s_and1_wrexec_b32);
		OP_TO_STRING_CASE(s_andn2_wrexec_b32);
		OP_TO_STRING_CASE(s_movrelsd_2_b32);
	default:
		return "UNKNOWN - SOP1";
	}

#undef OP_TO_STRING_CASE
}

enum class ESOP2Ops : uint16
{
	s_add_u32 = 0,
	s_sub_u32 = 1,
	s_add_i32 = 2,
	s_sub_i32 = 3,
	s_addc_u32 = 4,
	s_subb_u32 = 5,
	s_min_i32 = 6,
	s_min_u32 = 7,
	s_max_i32 = 8,
	s_max_u32 = 9,
	s_cselect_b32 = 10,
	s_cselect_b64 = 11,
	s_and_b32 = 14,
	s_and_b64 = 15,
	s_or_b32 = 16,
	s_or_b64 = 17,
	s_xor_b32 = 18,
	s_xor_b64 = 19,
	s_andn2_b32 = 20,
	s_andn2_b64 = 21,
	s_orn2_b32 = 22,
	s_orn2_b64 = 23,
	s_nand_b32 = 24,
	s_nand_b64 = 25,
	s_nor_b32 = 26,
	s_nor_b64 = 27,
	s_xnor_b32 = 28,
	s_xnor_b64 = 29,
	s_lshl_b32 = 30,
	s_lshl_b64 = 31,
	s_lshr_b32 = 32,
	s_lshr_b64 = 33,
	s_ashr_i32 = 34,
	s_ashr_i64 = 35,
	s_bfm_b32 = 36,
	s_bfm_b64 = 37,
	s_mul_i32 = 38,
	s_bfe_u32 = 39,
	s_bfe_i32 = 40,
	s_bfe_u64 = 41,
	s_bfe_i64 = 42,
	s_absdiff_i32 = 44,
	s_lshl1_add_u32 = 46,
	s_lshl2_add_u32 = 47,
	s_lshl3_add_u32 = 48,
	s_lshl4_add_u32 = 49,
	s_pack_ll_b32_b16 = 50,
	s_pack_lh_b32_b16 = 51,
	s_pack_hh_b32_b16 = 52,
	s_mul_hi_u32 = 53,
	s_mul_hi_i32 = 54,
};

const char* ToString(ESOP2Ops Op)
{
#define OP_TO_STRING_CASE(x) case ESOP2Ops::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(s_add_u32);
		OP_TO_STRING_CASE(s_sub_u32);
		OP_TO_STRING_CASE(s_add_i32);
		OP_TO_STRING_CASE(s_sub_i32);
		OP_TO_STRING_CASE(s_addc_u32);
		OP_TO_STRING_CASE(s_subb_u32);
		OP_TO_STRING_CASE(s_min_i32);
		OP_TO_STRING_CASE(s_min_u32);
		OP_TO_STRING_CASE(s_max_i32);
		OP_TO_STRING_CASE(s_max_u32);
		OP_TO_STRING_CASE(s_cselect_b32);
		OP_TO_STRING_CASE(s_cselect_b64);
		OP_TO_STRING_CASE(s_and_b32);
		OP_TO_STRING_CASE(s_and_b64);
		OP_TO_STRING_CASE(s_or_b32);
		OP_TO_STRING_CASE(s_or_b64);
		OP_TO_STRING_CASE(s_xor_b32);
		OP_TO_STRING_CASE(s_xor_b64);
		OP_TO_STRING_CASE(s_andn2_b32);
		OP_TO_STRING_CASE(s_andn2_b64);
		OP_TO_STRING_CASE(s_orn2_b32);
		OP_TO_STRING_CASE(s_orn2_b64);
		OP_TO_STRING_CASE(s_nand_b32);
		OP_TO_STRING_CASE(s_nand_b64);
		OP_TO_STRING_CASE(s_nor_b32);
		OP_TO_STRING_CASE(s_nor_b64);
		OP_TO_STRING_CASE(s_xnor_b32);
		OP_TO_STRING_CASE(s_xnor_b64);
		OP_TO_STRING_CASE(s_lshl_b32);
		OP_TO_STRING_CASE(s_lshl_b64);
		OP_TO_STRING_CASE(s_lshr_b32);
		OP_TO_STRING_CASE(s_lshr_b64);
		OP_TO_STRING_CASE(s_ashr_i32);
		OP_TO_STRING_CASE(s_ashr_i64);
		OP_TO_STRING_CASE(s_bfm_b32);
		OP_TO_STRING_CASE(s_bfm_b64);
		OP_TO_STRING_CASE(s_mul_i32);
		OP_TO_STRING_CASE(s_bfe_u32);
		OP_TO_STRING_CASE(s_bfe_i32);
		OP_TO_STRING_CASE(s_bfe_u64);
		OP_TO_STRING_CASE(s_bfe_i64);
		OP_TO_STRING_CASE(s_absdiff_i32);
		OP_TO_STRING_CASE(s_lshl1_add_u32);
		OP_TO_STRING_CASE(s_lshl2_add_u32);
		OP_TO_STRING_CASE(s_lshl3_add_u32);
		OP_TO_STRING_CASE(s_lshl4_add_u32);
		OP_TO_STRING_CASE(s_pack_ll_b32_b16);
		OP_TO_STRING_CASE(s_pack_lh_b32_b16);
		OP_TO_STRING_CASE(s_pack_hh_b32_b16);
		OP_TO_STRING_CASE(s_mul_hi_u32);
		OP_TO_STRING_CASE(s_mul_hi_i32);
	default:
		return "UNKNOWN - SOP2";
	}

#undef OP_TO_STRING_CASE
}

enum class ESOPKOps : uint16
{
	s_movk_i32 = 0,
	s_version = 1,
	s_cmovk_i32 = 2,
	s_cmpk_eq_i32 = 3,
	s_cmpk_lg_i32 = 4,
	s_cmpk_gt_i32 = 5,
	s_cmpk_ge_i32 = 6,
	s_cmpk_lt_i32 = 7,
	s_cmpk_le_i32 = 8,
	s_cmpk_eq_u32 = 9,
	s_cmpk_lg_u32 = 10,
	s_cmpk_gt_u32 = 11,
	s_cmpk_ge_u32 = 12,
	s_cmpk_lt_u32 = 13,
	s_cmpk_le_u32 = 14,
	s_addk_i32 = 15,
	s_mulk_i32 = 16,
	s_getreg_b32 = 18,
	s_setreg_b32 = 19,
	s_setreg_imm32_b32 = 21,
	s_call_b64 = 22,
	s_waitcnt_vscnt = 23,
	s_waitcnt_vmcnt = 24,
	s_waitcnt_expcnt = 25,
	s_waitcnt_lgkmcnt = 26,
	s_subvector_loop_begin = 27,
	s_subvector_loop_end = 28,
};

const char* ToString(ESOPKOps Op)
{
#define OP_TO_STRING_CASE(x) case ESOPKOps::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(s_movk_i32);
		OP_TO_STRING_CASE(s_version);
		OP_TO_STRING_CASE(s_cmovk_i32);
		OP_TO_STRING_CASE(s_cmpk_eq_i32);
		OP_TO_STRING_CASE(s_cmpk_lg_i32);
		OP_TO_STRING_CASE(s_cmpk_gt_i32);
		OP_TO_STRING_CASE(s_cmpk_ge_i32);
		OP_TO_STRING_CASE(s_cmpk_lt_i32);
		OP_TO_STRING_CASE(s_cmpk_le_i32);
		OP_TO_STRING_CASE(s_cmpk_eq_u32);
		OP_TO_STRING_CASE(s_cmpk_lg_u32);
		OP_TO_STRING_CASE(s_cmpk_gt_u32);
		OP_TO_STRING_CASE(s_cmpk_ge_u32);
		OP_TO_STRING_CASE(s_cmpk_lt_u32);
		OP_TO_STRING_CASE(s_cmpk_le_u32);
		OP_TO_STRING_CASE(s_addk_i32);
		OP_TO_STRING_CASE(s_mulk_i32);
		OP_TO_STRING_CASE(s_getreg_b32);
		OP_TO_STRING_CASE(s_setreg_b32);
		OP_TO_STRING_CASE(s_setreg_imm32_b32);
		OP_TO_STRING_CASE(s_call_b64);
		OP_TO_STRING_CASE(s_waitcnt_vscnt);
		OP_TO_STRING_CASE(s_waitcnt_vmcnt);
		OP_TO_STRING_CASE(s_waitcnt_expcnt);
		OP_TO_STRING_CASE(s_waitcnt_lgkmcnt);
		OP_TO_STRING_CASE(s_subvector_loop_begin);
		OP_TO_STRING_CASE(s_subvector_loop_end);
	default:
		return "UNKNOWN - SOPK";
	}

#undef OP_TO_STRING_CASE
}

enum class ESOPPOps : uint16
{
	s_nop = 0,
	s_endpgm = 1,
	s_branch = 2,
	s_wakeup = 3,
	s_cbranch_scc0 = 4,
	s_cbranch_scc1 = 5,
	s_cbranch_vccz = 6,
	s_cbranch_vccnz = 7,
	s_cbranch_execz = 8,
	s_cbranch_execnz = 9,
	s_barrier = 10,
	s_setkill = 11,
	s_waitcnt = 12,
	s_sethalt = 13,
	s_sleep = 14,
	s_setprio = 15,
	s_sendmsg = 16,
	s_sendmsghalt = 17,
	s_trap = 18,
	s_icache_inv = 19,
	s_incperflevel = 20,
	s_decperflevel = 21,
	s_ttracedata = 22,
	s_cbranch_cdbgsys = 23,
	s_cbranch_cdbguser = 24,
	s_cbranch_cdbgsys_or_user = 25,
	s_cbranch_cdbgsys_and_user = 26,
	s_endpgm_saved = 27,
	s_endpgm_ordered_ps_done = 30,
	s_code_end = 31,
	s_inst_prefetch = 32,
	s_clause = 33,
	s_waitcnt_depctr = 35,
	s_round_mode = 36,
	s_denorm_mode = 37,
	s_ttracedata_imm = 40,
};

const char* ToString(ESOPPOps Op)
{
#define OP_TO_STRING_CASE(x) case ESOPPOps::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(s_nop);
		OP_TO_STRING_CASE(s_endpgm);
		OP_TO_STRING_CASE(s_branch);
		OP_TO_STRING_CASE(s_wakeup);
		OP_TO_STRING_CASE(s_cbranch_scc0);
		OP_TO_STRING_CASE(s_cbranch_scc1);
		OP_TO_STRING_CASE(s_cbranch_vccz);
		OP_TO_STRING_CASE(s_cbranch_vccnz);
		OP_TO_STRING_CASE(s_cbranch_execz);
		OP_TO_STRING_CASE(s_cbranch_execnz);
		OP_TO_STRING_CASE(s_barrier);
		OP_TO_STRING_CASE(s_setkill);
		OP_TO_STRING_CASE(s_waitcnt);
		OP_TO_STRING_CASE(s_sethalt);
		OP_TO_STRING_CASE(s_sleep);
		OP_TO_STRING_CASE(s_setprio);
		OP_TO_STRING_CASE(s_sendmsg);
		OP_TO_STRING_CASE(s_sendmsghalt);
		OP_TO_STRING_CASE(s_trap);
		OP_TO_STRING_CASE(s_icache_inv);
		OP_TO_STRING_CASE(s_incperflevel);
		OP_TO_STRING_CASE(s_decperflevel);
		OP_TO_STRING_CASE(s_ttracedata);
		OP_TO_STRING_CASE(s_cbranch_cdbgsys);
		OP_TO_STRING_CASE(s_cbranch_cdbguser);
		OP_TO_STRING_CASE(s_cbranch_cdbgsys_or_user);
		OP_TO_STRING_CASE(s_cbranch_cdbgsys_and_user);
		OP_TO_STRING_CASE(s_endpgm_saved);
		OP_TO_STRING_CASE(s_endpgm_ordered_ps_done);
		OP_TO_STRING_CASE(s_code_end);
		OP_TO_STRING_CASE(s_inst_prefetch);
		OP_TO_STRING_CASE(s_clause);
		OP_TO_STRING_CASE(s_waitcnt_depctr);
		OP_TO_STRING_CASE(s_round_mode);
		OP_TO_STRING_CASE(s_denorm_mode);
		OP_TO_STRING_CASE(s_ttracedata_imm);
	default:
		return "UNKNOWN - SOPP";
	}

#undef OP_TO_STRING_CASE
}

enum class ESMEMOps : uint16
{
	s_load_dword = 0,
	s_load_dwordx2 = 1,
	s_load_dwordx4 = 2,
	s_load_dwordx8 = 3,
	s_load_dwordx16 = 4,
	s_buffer_load_dword = 8,
	s_buffer_load_dwordx2 = 9,
	s_buffer_load_dwordx4 = 10,
	s_buffer_load_dwordx8 = 11,
	s_buffer_load_dwordx16 = 12,
	s_gl1_inv = 31,
	s_dcache_inv = 32,
	s_memtime = 36,
	s_memrealtime = 37,
	s_atc_probe = 38,
	s_atc_probe_buffer = 39,
};

const char* ToString(ESMEMOps Op)
{
#define OP_TO_STRING_CASE(x) case ESMEMOps::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(s_load_dword);
		OP_TO_STRING_CASE(s_load_dwordx2);
		OP_TO_STRING_CASE(s_load_dwordx4);
		OP_TO_STRING_CASE(s_load_dwordx8);
		OP_TO_STRING_CASE(s_load_dwordx16);
		OP_TO_STRING_CASE(s_buffer_load_dword);
		OP_TO_STRING_CASE(s_buffer_load_dwordx2);
		OP_TO_STRING_CASE(s_buffer_load_dwordx4);
		OP_TO_STRING_CASE(s_buffer_load_dwordx8);
		OP_TO_STRING_CASE(s_buffer_load_dwordx16);
		OP_TO_STRING_CASE(s_gl1_inv);
		OP_TO_STRING_CASE(s_dcache_inv);
		OP_TO_STRING_CASE(s_memtime);
		OP_TO_STRING_CASE(s_memrealtime);
		OP_TO_STRING_CASE(s_atc_probe);
		OP_TO_STRING_CASE(s_atc_probe_buffer);
	default:
		return "UNKNOWN - SMEM";
	}

#undef OP_TO_STRING_CASE
}

enum class ESOPCOps : uint16
{
	s_cmp_eq_i32 = 0,
	s_cmp_lg_i32 = 1,
	s_cmp_gt_i32 = 2,
	s_cmp_ge_i32 = 3,
	s_cmp_lt_i32 = 4,
	s_cmp_le_i32 = 5,
	s_cmp_eq_u32 = 6,
	s_cmp_lg_u32 = 7,
	s_cmp_gt_u32 = 8,
	s_cmp_ge_u32 = 9,
	s_cmp_lt_u32 = 10,
	s_cmp_le_u32 = 11,
	s_bitcmp0_b32 = 12,
	s_bitcmp1_b32 = 13,
	s_bitcmp0_b64 = 14,
	s_bitcmp1_b64 = 15,
	s_cmp_eq_u64 = 18,
	s_cmp_lg_u64 = 19,
};

const char* ToString(ESOPCOps Op)
{
#define OP_TO_STRING_CASE(x) case ESOPCOps::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(s_cmp_eq_i32);
		OP_TO_STRING_CASE(s_cmp_lg_i32);
		OP_TO_STRING_CASE(s_cmp_gt_i32);
		OP_TO_STRING_CASE(s_cmp_ge_i32);
		OP_TO_STRING_CASE(s_cmp_lt_i32);
		OP_TO_STRING_CASE(s_cmp_le_i32);
		OP_TO_STRING_CASE(s_cmp_eq_u32);
		OP_TO_STRING_CASE(s_cmp_lg_u32);
		OP_TO_STRING_CASE(s_cmp_gt_u32);
		OP_TO_STRING_CASE(s_cmp_ge_u32);
		OP_TO_STRING_CASE(s_cmp_lt_u32);
		OP_TO_STRING_CASE(s_cmp_le_u32);
		OP_TO_STRING_CASE(s_bitcmp0_b32);
		OP_TO_STRING_CASE(s_bitcmp1_b32);
		OP_TO_STRING_CASE(s_bitcmp0_b64);
		OP_TO_STRING_CASE(s_bitcmp1_b64);
		OP_TO_STRING_CASE(s_cmp_eq_u64);
		OP_TO_STRING_CASE(s_cmp_lg_u64);
	default:
		return "UNKNOWN - SOPC";
	}

#undef OP_TO_STRING_CASE
}

enum class EVOP1Ops : uint16
{
	v_nop = 0,
	v_mov_b32 = 1,
	v_readfirstlane_b32 = 2,
	v_cvt_i32_f64 = 3,
	v_cvt_f64_i32 = 4,
	v_cvt_f32_i32 = 5,
	v_cvt_f32_u32 = 6,
	v_cvt_u32_f32 = 7,
	v_cvt_i32_f32 = 8,
	v_cvt_f16_f32 = 10,
	v_cvt_f32_f16 = 11,
	v_cvt_rpi_i32_f32 = 12,
	v_cvt_flr_i32_f32 = 13,
	v_cvt_off_f32_i4 = 14,
	v_cvt_f32_f64 = 15,
	v_cvt_f64_f32 = 16,
	v_cvt_f32_ubyte0 = 17,
	v_cvt_f32_ubyte1 = 18,
	v_cvt_f32_ubyte2 = 19,
	v_cvt_f32_ubyte3 = 20,
	v_cvt_u32_f64 = 21,
	v_cvt_f64_u32 = 22,
	v_trunc_f64 = 23,
	v_ceil_f64 = 24,
	v_rndne_f64 = 25,
	v_floor_f64 = 26,
	v_pipeflush = 27,
	v_fract_f32 = 32,
	v_trunc_f32 = 33,
	v_ceil_f32 = 34,
	v_rndne_f32 = 35,
	v_floor_f32 = 36,
	v_exp_f32 = 37,
	v_log_f32 = 39,
	v_rcp_f32 = 42,
	v_rcp_iflag_f32 = 43,
	v_rsq_f32 = 46,
	v_rcp_f64 = 47,
	v_rsq_f64 = 49,
	v_sqrt_f32 = 51,
	v_sqrt_f64 = 52,
	v_sin_f32 = 53,
	v_cos_f32 = 54,
	v_not_b32 = 55,
	v_bfrev_b32 = 56,
	v_ffbh_u32 = 57,
	v_ffbl_b32 = 58,
	v_ffbh_i32 = 59,
	v_frexp_exp_i32_f64 = 60,
	v_frexp_mant_f64 = 61,
	v_fract_f64 = 62,
	v_frexp_exp_i32_f32 = 63,
	v_frexp_mant_f32 = 64,
	v_clrexcp = 65,
	v_movreld_b32 = 66,
	v_movrels_b32 = 67,
	v_movrelsd_b32 = 68,
	v_movrelsd_2_b32 = 72,
	v_cvt_f16_u16 = 80,
	v_cvt_f16_i16 = 81,
	v_cvt_u16_f16 = 82,
	v_cvt_i16_f16 = 83,
	v_rcp_f16 = 84,
	v_sqrt_f16 = 85,
	v_rsq_f16 = 86,
	v_log_f16 = 87,
	v_exp_f16 = 88,
	v_frexp_mant_f16 = 89,
	v_frexp_exp_i16_f16 = 90,
	v_floor_f16 = 91,
	v_ceil_f16 = 92,
	v_trunc_f16 = 93,
	v_rndne_f16 = 94,
	v_fract_f16 = 95,
	v_sin_f16 = 96,
	v_cos_f16 = 97,
	v_sat_pk_u8_i16 = 98,
	v_cvt_norm_i16_f16 = 99,
	v_cvt_norm_u16_f16 = 100,
	v_swap_b32 = 101,
	v_swapref_b32 = 104,
};

const char* ToString(EVOP1Ops Op)
{
#define OP_TO_STRING_CASE(x) case EVOP1Ops::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(v_nop);
		OP_TO_STRING_CASE(v_mov_b32);
		OP_TO_STRING_CASE(v_readfirstlane_b32);
		OP_TO_STRING_CASE(v_cvt_i32_f64);
		OP_TO_STRING_CASE(v_cvt_f64_i32);
		OP_TO_STRING_CASE(v_cvt_f32_i32);
		OP_TO_STRING_CASE(v_cvt_f32_u32);
		OP_TO_STRING_CASE(v_cvt_u32_f32);
		OP_TO_STRING_CASE(v_cvt_i32_f32);
		OP_TO_STRING_CASE(v_cvt_f16_f32);
		OP_TO_STRING_CASE(v_cvt_f32_f16);
		OP_TO_STRING_CASE(v_cvt_rpi_i32_f32);
		OP_TO_STRING_CASE(v_cvt_flr_i32_f32);
		OP_TO_STRING_CASE(v_cvt_off_f32_i4);
		OP_TO_STRING_CASE(v_cvt_f32_f64);
		OP_TO_STRING_CASE(v_cvt_f64_f32);
		OP_TO_STRING_CASE(v_cvt_f32_ubyte0);
		OP_TO_STRING_CASE(v_cvt_f32_ubyte1);
		OP_TO_STRING_CASE(v_cvt_f32_ubyte2);
		OP_TO_STRING_CASE(v_cvt_f32_ubyte3);
		OP_TO_STRING_CASE(v_cvt_u32_f64);
		OP_TO_STRING_CASE(v_cvt_f64_u32);
		OP_TO_STRING_CASE(v_trunc_f64);
		OP_TO_STRING_CASE(v_ceil_f64);
		OP_TO_STRING_CASE(v_rndne_f64);
		OP_TO_STRING_CASE(v_floor_f64);
		OP_TO_STRING_CASE(v_pipeflush);
		OP_TO_STRING_CASE(v_fract_f32);
		OP_TO_STRING_CASE(v_trunc_f32);
		OP_TO_STRING_CASE(v_ceil_f32);
		OP_TO_STRING_CASE(v_rndne_f32);
		OP_TO_STRING_CASE(v_floor_f32);
		OP_TO_STRING_CASE(v_exp_f32);
		OP_TO_STRING_CASE(v_log_f32);
		OP_TO_STRING_CASE(v_rcp_f32);
		OP_TO_STRING_CASE(v_rcp_iflag_f32);
		OP_TO_STRING_CASE(v_rsq_f32);
		OP_TO_STRING_CASE(v_rcp_f64);
		OP_TO_STRING_CASE(v_rsq_f64);
		OP_TO_STRING_CASE(v_sqrt_f32);
		OP_TO_STRING_CASE(v_sqrt_f64);
		OP_TO_STRING_CASE(v_sin_f32);
		OP_TO_STRING_CASE(v_cos_f32);
		OP_TO_STRING_CASE(v_not_b32);
		OP_TO_STRING_CASE(v_bfrev_b32);
		OP_TO_STRING_CASE(v_ffbh_u32);
		OP_TO_STRING_CASE(v_ffbl_b32);
		OP_TO_STRING_CASE(v_ffbh_i32);
		OP_TO_STRING_CASE(v_frexp_exp_i32_f64);
		OP_TO_STRING_CASE(v_frexp_mant_f64);
		OP_TO_STRING_CASE(v_fract_f64);
		OP_TO_STRING_CASE(v_frexp_exp_i32_f32);
		OP_TO_STRING_CASE(v_frexp_mant_f32);
		OP_TO_STRING_CASE(v_clrexcp);
		OP_TO_STRING_CASE(v_movreld_b32);
		OP_TO_STRING_CASE(v_movrels_b32);
		OP_TO_STRING_CASE(v_movrelsd_b32);
		OP_TO_STRING_CASE(v_movrelsd_2_b32);
		OP_TO_STRING_CASE(v_cvt_f16_u16);
		OP_TO_STRING_CASE(v_cvt_f16_i16);
		OP_TO_STRING_CASE(v_cvt_u16_f16);
		OP_TO_STRING_CASE(v_cvt_i16_f16);
		OP_TO_STRING_CASE(v_rcp_f16);
		OP_TO_STRING_CASE(v_sqrt_f16);
		OP_TO_STRING_CASE(v_rsq_f16);
		OP_TO_STRING_CASE(v_log_f16);
		OP_TO_STRING_CASE(v_exp_f16);
		OP_TO_STRING_CASE(v_frexp_mant_f16);
		OP_TO_STRING_CASE(v_frexp_exp_i16_f16);
		OP_TO_STRING_CASE(v_floor_f16);
		OP_TO_STRING_CASE(v_ceil_f16);
		OP_TO_STRING_CASE(v_trunc_f16);
		OP_TO_STRING_CASE(v_rndne_f16);
		OP_TO_STRING_CASE(v_fract_f16);
		OP_TO_STRING_CASE(v_sin_f16);
		OP_TO_STRING_CASE(v_cos_f16);
		OP_TO_STRING_CASE(v_sat_pk_u8_i16);
		OP_TO_STRING_CASE(v_cvt_norm_i16_f16);
		OP_TO_STRING_CASE(v_cvt_norm_u16_f16);
		OP_TO_STRING_CASE(v_swap_b32);
		OP_TO_STRING_CASE(v_swapref_b32);
	default:
		return "UNKNOWN - VOP1";
	}

#undef OP_TO_STRING_CASE
}

enum class EVOP3POps : uint16
{
	v_pk_mad_i16 = 0,
	v_pk_mul_lo_u16 = 1,
	v_pk_add_i16 = 2,
	v_pk_sub_i16 = 3,
	v_pk_lshlrev_b16 = 4,
	v_pk_lshrrev_b16 = 5,
	v_pk_ashrrev_i16 = 6,
	v_pk_max_i16 = 7,
	v_pk_min_i16 = 8,
	v_pk_mad_u16 = 9,
	v_pk_add_u16 = 10,
	v_pk_sub_u16 = 11,
	v_pk_max_u16 = 12,
	v_pk_min_u16 = 13,
	v_pk_fma_f16 = 14,
	v_pk_add_f16 = 15,
	v_pk_mul_f16 = 16,
	v_pk_min_f16 = 17,
	v_pk_max_f16 = 18,
	v_fma_mix_f32 = 32,
	v_fma_mixlo_f16 = 33,
	v_fma_mixhi_f16 = 34,
};

const char* ToString(EVOP3POps Op)
{
#define OP_TO_STRING_CASE(x) case EVOP3POps::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(v_pk_mad_i16);
		OP_TO_STRING_CASE(v_pk_mul_lo_u16);
		OP_TO_STRING_CASE(v_pk_add_i16);
		OP_TO_STRING_CASE(v_pk_sub_i16);
		OP_TO_STRING_CASE(v_pk_lshlrev_b16);
		OP_TO_STRING_CASE(v_pk_lshrrev_b16);
		OP_TO_STRING_CASE(v_pk_ashrrev_i16);
		OP_TO_STRING_CASE(v_pk_max_i16);
		OP_TO_STRING_CASE(v_pk_min_i16);
		OP_TO_STRING_CASE(v_pk_mad_u16);
		OP_TO_STRING_CASE(v_pk_add_u16);
		OP_TO_STRING_CASE(v_pk_sub_u16);
		OP_TO_STRING_CASE(v_pk_max_u16);
		OP_TO_STRING_CASE(v_pk_min_u16);
		OP_TO_STRING_CASE(v_pk_fma_f16);
		OP_TO_STRING_CASE(v_pk_add_f16);
		OP_TO_STRING_CASE(v_pk_mul_f16);
		OP_TO_STRING_CASE(v_pk_min_f16);
		OP_TO_STRING_CASE(v_pk_max_f16);
		OP_TO_STRING_CASE(v_fma_mix_f32);
		OP_TO_STRING_CASE(v_fma_mixlo_f16);
		OP_TO_STRING_CASE(v_fma_mixhi_f16);
	default:
		return "UNKNOWN - VOP3P";
	}

#undef OP_TO_STRING_CASE
}

enum class EVOP2Ops : uint16
{
	v_cndmask_b32 = 1,
	v_dot2c_f32_f16 = 2,
	v_add_f32 = 3,
	v_sub_f32 = 4,
	v_subrev_f32 = 5,
	v_fmac_legacy_f32 = 6,
	v_mul_legacy_f32 = 7,
	v_mul_f32 = 8,
	v_mul_i32_i24 = 9,
	v_mul_hi_i32_i24 = 10,
	v_mul_u32_u24 = 11,
	v_mul_hi_u32_u24 = 12,
	v_dot4c_i32_i8 = 13,
	v_min_f32 = 15,
	v_max_f32 = 16,
	v_min_i32 = 17,
	v_max_i32 = 18,
	v_min_u32 = 19,
	v_max_u32 = 20,
	v_lshrrev_b32 = 22,
	v_ashrrev_i32 = 24,
	v_lshlrev_b32 = 26,
	v_and_b32 = 27,
	v_or_b32 = 28,
	v_xor_b32 = 29,
	v_xnor_b32 = 30,
	v_mac_f32 = 31,
	v_madmk_f32 = 32,
	v_madak_f32 = 33,
	v_add_nc_u32 = 37,
	v_sub_nc_u32 = 38,
	v_subrev_nc_u32 = 39,
	v_add_co_ci_u32 = 40,
	v_sub_co_ci_u32 = 41,
	v_subrev_co_ci_u32 = 42,
	v_fmac_f32 = 43,
	v_fmamk_f32 = 44,
	v_fmaak_f32 = 45,
	v_cvt_pkrtz_f16_f32 = 47,
	v_add_f16 = 50,
	v_sub_f16 = 51,
	v_subrev_f16 = 52,
	v_mul_f16 = 53,
	v_fmac_f16 = 54,
	v_fmamk_f16 = 55,
	v_fmaak_f16 = 56,
	v_max_f16 = 57,
	v_min_f16 = 58,
	v_ldexp_f16 = 59,
	v_pk_fmac_f16 = 60,
};

const char* ToString(EVOP2Ops Op)
{
#define OP_TO_STRING_CASE(x) case EVOP2Ops::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(v_cndmask_b32);
		OP_TO_STRING_CASE(v_dot2c_f32_f16);
		OP_TO_STRING_CASE(v_add_f32);
		OP_TO_STRING_CASE(v_sub_f32);
		OP_TO_STRING_CASE(v_subrev_f32);
		OP_TO_STRING_CASE(v_fmac_legacy_f32);
		OP_TO_STRING_CASE(v_mul_legacy_f32);
		OP_TO_STRING_CASE(v_mul_f32);
		OP_TO_STRING_CASE(v_mul_i32_i24);
		OP_TO_STRING_CASE(v_mul_hi_i32_i24);
		OP_TO_STRING_CASE(v_mul_u32_u24);
		OP_TO_STRING_CASE(v_mul_hi_u32_u24);
		OP_TO_STRING_CASE(v_dot4c_i32_i8);
		OP_TO_STRING_CASE(v_min_f32);
		OP_TO_STRING_CASE(v_max_f32);
		OP_TO_STRING_CASE(v_min_i32);
		OP_TO_STRING_CASE(v_max_i32);
		OP_TO_STRING_CASE(v_min_u32);
		OP_TO_STRING_CASE(v_max_u32);
		OP_TO_STRING_CASE(v_lshrrev_b32);
		OP_TO_STRING_CASE(v_ashrrev_i32);
		OP_TO_STRING_CASE(v_lshlrev_b32);
		OP_TO_STRING_CASE(v_and_b32);
		OP_TO_STRING_CASE(v_or_b32);
		OP_TO_STRING_CASE(v_xor_b32);
		OP_TO_STRING_CASE(v_xnor_b32);
		OP_TO_STRING_CASE(v_mac_f32);
		OP_TO_STRING_CASE(v_madmk_f32);
		OP_TO_STRING_CASE(v_madak_f32);
		OP_TO_STRING_CASE(v_add_nc_u32);
		OP_TO_STRING_CASE(v_sub_nc_u32);
		OP_TO_STRING_CASE(v_subrev_nc_u32);
		OP_TO_STRING_CASE(v_add_co_ci_u32);
		OP_TO_STRING_CASE(v_sub_co_ci_u32);
		OP_TO_STRING_CASE(v_subrev_co_ci_u32);
		OP_TO_STRING_CASE(v_fmac_f32);
		OP_TO_STRING_CASE(v_fmamk_f32);
		OP_TO_STRING_CASE(v_fmaak_f32);
		OP_TO_STRING_CASE(v_cvt_pkrtz_f16_f32);
		OP_TO_STRING_CASE(v_add_f16);
		OP_TO_STRING_CASE(v_sub_f16);
		OP_TO_STRING_CASE(v_subrev_f16);
		OP_TO_STRING_CASE(v_mul_f16);
		OP_TO_STRING_CASE(v_fmac_f16);
		OP_TO_STRING_CASE(v_fmamk_f16);
		OP_TO_STRING_CASE(v_fmaak_f16);
		OP_TO_STRING_CASE(v_max_f16);
		OP_TO_STRING_CASE(v_min_f16);
		OP_TO_STRING_CASE(v_ldexp_f16);
		OP_TO_STRING_CASE(v_pk_fmac_f16);
	default:
		return "UNKNOWN - VOP2";
	}

#undef OP_TO_STRING_CASE
}

enum class EVOPCOps : uint16
{
	v_cmp_f_f32 = 0,
	v_cmp_lt_f32 = 1,
	v_cmp_eq_f32 = 2,
	v_cmp_le_f32 = 3,
	v_cmp_gt_f32 = 4,
	v_cmp_lg_f32 = 5,
	v_cmp_ge_f32 = 6,
	v_cmp_o_f32 = 7,
	v_cmp_u_f32 = 8,
	v_cmp_nge_f32 = 9,
	v_cmp_nlg_f32 = 10,
	v_cmp_ngt_f32 = 11,
	v_cmp_nle_f32 = 12,
	v_cmp_neq_f32 = 13,
	v_cmp_nlt_f32 = 14,
	v_cmp_tru_f32 = 15,
	v_cmpx_f_f32 = 16,
	v_cmpx_lt_f32 = 17,
	v_cmpx_eq_f32 = 18,
	v_cmpx_le_f32 = 19,
	v_cmpx_gt_f32 = 20,
	v_cmpx_lg_f32 = 21,
	v_cmpx_ge_f32 = 22,
	v_cmpx_o_f32 = 23,
	v_cmpx_u_f32 = 24,
	v_cmpx_nge_f32 = 25,
	v_cmpx_nlg_f32 = 26,
	v_cmpx_ngt_f32 = 27,
	v_cmpx_nle_f32 = 28,
	v_cmpx_neq_f32 = 29,
	v_cmpx_nlt_f32 = 30,
	v_cmpx_tru_f32 = 31,
	v_cmp_f_f64 = 32,
	v_cmp_lt_f64 = 33,
	v_cmp_eq_f64 = 34,
	v_cmp_le_f64 = 35,
	v_cmp_gt_f64 = 36,
	v_cmp_lg_f64 = 37,
	v_cmp_ge_f64 = 38,
	v_cmp_o_f64 = 39,
	v_cmp_u_f64 = 40,
	v_cmp_nge_f64 = 41,
	v_cmp_nlg_f64 = 42,
	v_cmp_ngt_f64 = 43,
	v_cmp_nle_f64 = 44,
	v_cmp_neq_f64 = 45,
	v_cmp_nlt_f64 = 46,
	v_cmp_tru_f64 = 47,
	v_cmpx_f_f64 = 48,
	v_cmpx_lt_f64 = 49,
	v_cmpx_eq_f64 = 50,
	v_cmpx_le_f64 = 51,
	v_cmpx_gt_f64 = 52,
	v_cmpx_lg_f64 = 53,
	v_cmpx_ge_f64 = 54,
	v_cmpx_o_f64 = 55,
	v_cmpx_u_f64 = 56,
	v_cmpx_nge_f64 = 57,
	v_cmpx_nlg_f64 = 58,
	v_cmpx_ngt_f64 = 59,
	v_cmpx_nle_f64 = 60,
	v_cmpx_neq_f64 = 61,
	v_cmpx_nlt_f64 = 62,
	v_cmpx_tru_f64 = 63,
	v_cmp_f_i32 = 128,
	v_cmp_lt_i32 = 129,
	v_cmp_eq_i32 = 130,
	v_cmp_le_i32 = 131,
	v_cmp_gt_i32 = 132,
	v_cmp_ne_i32 = 133,
	v_cmp_ge_i32 = 134,
	v_cmp_t_i32 = 135,
	v_cmp_class_f32 = 136,
	v_cmp_lt_i16 = 137,
	v_cmp_eq_i16 = 138,
	v_cmp_le_i16 = 139,
	v_cmp_gt_i16 = 140,
	v_cmp_ne_i16 = 141,
	v_cmp_ge_i16 = 142,
	v_cmp_class_f16 = 143,
	v_cmpx_f_i32 = 144,
	v_cmpx_lt_i32 = 145,
	v_cmpx_eq_i32 = 146,
	v_cmpx_le_i32 = 147,
	v_cmpx_gt_i32 = 148,
	v_cmpx_ne_i32 = 149,
	v_cmpx_ge_i32 = 150,
	v_cmpx_t_i32 = 151,
	v_cmpx_class_f32 = 152,
	v_cmpx_lt_i16 = 153,
	v_cmpx_eq_i16 = 154,
	v_cmpx_le_i16 = 155,
	v_cmpx_gt_i16 = 156,
	v_cmpx_ne_i16 = 157,
	v_cmpx_ge_i16 = 158,
	v_cmpx_class_f16 = 159,
	v_cmp_f_i64 = 160,
	v_cmp_lt_i64 = 161,
	v_cmp_eq_i64 = 162,
	v_cmp_le_i64 = 163,
	v_cmp_gt_i64 = 164,
	v_cmp_ne_i64 = 165,
	v_cmp_ge_i64 = 166,
	v_cmp_t_i64 = 167,
	v_cmp_class_f64 = 168,
	v_cmp_lt_u16 = 169,
	v_cmp_eq_u16 = 170,
	v_cmp_le_u16 = 171,
	v_cmp_gt_u16 = 172,
	v_cmp_ne_u16 = 173,
	v_cmp_ge_u16 = 174,
	v_cmpx_f_i64 = 176,
	v_cmpx_lt_i64 = 177,
	v_cmpx_eq_i64 = 178,
	v_cmpx_le_i64 = 179,
	v_cmpx_gt_i64 = 180,
	v_cmpx_ne_i64 = 181,
	v_cmpx_ge_i64 = 182,
	v_cmpx_t_i64 = 183,
	v_cmpx_class_f64 = 184,
	v_cmpx_lt_u16 = 185,
	v_cmpx_eq_u16 = 186,
	v_cmpx_le_u16 = 187,
	v_cmpx_gt_u16 = 188,
	v_cmpx_ne_u16 = 189,
	v_cmpx_ge_u16 = 190,
	v_cmp_f_u32 = 192,
	v_cmp_lt_u32 = 193,
	v_cmp_eq_u32 = 194,
	v_cmp_le_u32 = 195,
	v_cmp_gt_u32 = 196,
	v_cmp_ne_u32 = 197,
	v_cmp_ge_u32 = 198,
	v_cmp_t_u32 = 199,
	v_cmp_f_f16 = 200,
	v_cmp_lt_f16 = 201,
	v_cmp_eq_f16 = 202,
	v_cmp_le_f16 = 203,
	v_cmp_gt_f16 = 204,
	v_cmp_lg_f16 = 205,
	v_cmp_ge_f16 = 206,
	v_cmp_o_f16 = 207,
	v_cmpx_f_u32 = 208,
	v_cmpx_lt_u32 = 209,
	v_cmpx_eq_u32 = 210,
	v_cmpx_le_u32 = 211,
	v_cmpx_gt_u32 = 212,
	v_cmpx_ne_u32 = 213,
	v_cmpx_ge_u32 = 214,
	v_cmpx_t_u32 = 215,
	v_cmpx_f_f16 = 216,
	v_cmpx_lt_f16 = 217,
	v_cmpx_eq_f16 = 218,
	v_cmpx_le_f16 = 219,
	v_cmpx_gt_f16 = 220,
	v_cmpx_lg_f16 = 221,
	v_cmpx_ge_f16 = 222,
	v_cmpx_o_f16 = 223,
	v_cmp_f_u64 = 224,
	v_cmp_lt_u64 = 225,
	v_cmp_eq_u64 = 226,
	v_cmp_le_u64 = 227,
	v_cmp_gt_u64 = 228,
	v_cmp_ne_u64 = 229,
	v_cmp_ge_u64 = 230,
	v_cmp_t_u64 = 231,
	v_cmp_u_f16 = 232,
	v_cmp_nge_f16 = 233,
	v_cmp_nlg_f16 = 234,
	v_cmp_ngt_f16 = 235,
	v_cmp_nle_f16 = 236,
	v_cmp_neq_f16 = 237,
	v_cmp_nlt_f16 = 238,
	v_cmp_tru_f16 = 239,
	v_cmpx_f_u64 = 240,
	v_cmpx_lt_u64 = 241,
	v_cmpx_eq_u64 = 242,
	v_cmpx_le_u64 = 243,
	v_cmpx_gt_u64 = 244,
	v_cmpx_ne_u64 = 245,
	v_cmpx_ge_u64 = 246,
	v_cmpx_t_u64 = 247,
	v_cmpx_u_f16 = 248,
	v_cmpx_nge_f16 = 249,
	v_cmpx_nlg_f16 = 250,
	v_cmpx_ngt_f16 = 251,
	v_cmpx_nle_f16 = 252,
	v_cmpx_neq_f16 = 253,
	v_cmpx_nlt_f16 = 254,
	v_cmpx_tru_f16 = 255,
};

const char* ToString(EVOPCOps Op)
{
#define OP_TO_STRING_CASE(x) case EVOPCOps::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(v_cmp_f_f32);
		OP_TO_STRING_CASE(v_cmp_lt_f32);
		OP_TO_STRING_CASE(v_cmp_eq_f32);
		OP_TO_STRING_CASE(v_cmp_le_f32);
		OP_TO_STRING_CASE(v_cmp_gt_f32);
		OP_TO_STRING_CASE(v_cmp_lg_f32);
		OP_TO_STRING_CASE(v_cmp_ge_f32);
		OP_TO_STRING_CASE(v_cmp_o_f32);
		OP_TO_STRING_CASE(v_cmp_u_f32);
		OP_TO_STRING_CASE(v_cmp_nge_f32);
		OP_TO_STRING_CASE(v_cmp_nlg_f32);
		OP_TO_STRING_CASE(v_cmp_ngt_f32);
		OP_TO_STRING_CASE(v_cmp_nle_f32);
		OP_TO_STRING_CASE(v_cmp_neq_f32);
		OP_TO_STRING_CASE(v_cmp_nlt_f32);
		OP_TO_STRING_CASE(v_cmp_tru_f32);
		OP_TO_STRING_CASE(v_cmpx_f_f32);
		OP_TO_STRING_CASE(v_cmpx_lt_f32);
		OP_TO_STRING_CASE(v_cmpx_eq_f32);
		OP_TO_STRING_CASE(v_cmpx_le_f32);
		OP_TO_STRING_CASE(v_cmpx_gt_f32);
		OP_TO_STRING_CASE(v_cmpx_lg_f32);
		OP_TO_STRING_CASE(v_cmpx_ge_f32);
		OP_TO_STRING_CASE(v_cmpx_o_f32);
		OP_TO_STRING_CASE(v_cmpx_u_f32);
		OP_TO_STRING_CASE(v_cmpx_nge_f32);
		OP_TO_STRING_CASE(v_cmpx_nlg_f32);
		OP_TO_STRING_CASE(v_cmpx_ngt_f32);
		OP_TO_STRING_CASE(v_cmpx_nle_f32);
		OP_TO_STRING_CASE(v_cmpx_neq_f32);
		OP_TO_STRING_CASE(v_cmpx_nlt_f32);
		OP_TO_STRING_CASE(v_cmpx_tru_f32);
		OP_TO_STRING_CASE(v_cmp_f_f64);
		OP_TO_STRING_CASE(v_cmp_lt_f64);
		OP_TO_STRING_CASE(v_cmp_eq_f64);
		OP_TO_STRING_CASE(v_cmp_le_f64);
		OP_TO_STRING_CASE(v_cmp_gt_f64);
		OP_TO_STRING_CASE(v_cmp_lg_f64);
		OP_TO_STRING_CASE(v_cmp_ge_f64);
		OP_TO_STRING_CASE(v_cmp_o_f64);
		OP_TO_STRING_CASE(v_cmp_u_f64);
		OP_TO_STRING_CASE(v_cmp_nge_f64);
		OP_TO_STRING_CASE(v_cmp_nlg_f64);
		OP_TO_STRING_CASE(v_cmp_ngt_f64);
		OP_TO_STRING_CASE(v_cmp_nle_f64);
		OP_TO_STRING_CASE(v_cmp_neq_f64);
		OP_TO_STRING_CASE(v_cmp_nlt_f64);
		OP_TO_STRING_CASE(v_cmp_tru_f64);
		OP_TO_STRING_CASE(v_cmpx_f_f64);
		OP_TO_STRING_CASE(v_cmpx_lt_f64);
		OP_TO_STRING_CASE(v_cmpx_eq_f64);
		OP_TO_STRING_CASE(v_cmpx_le_f64);
		OP_TO_STRING_CASE(v_cmpx_gt_f64);
		OP_TO_STRING_CASE(v_cmpx_lg_f64);
		OP_TO_STRING_CASE(v_cmpx_ge_f64);
		OP_TO_STRING_CASE(v_cmpx_o_f64);
		OP_TO_STRING_CASE(v_cmpx_u_f64);
		OP_TO_STRING_CASE(v_cmpx_nge_f64);
		OP_TO_STRING_CASE(v_cmpx_nlg_f64);
		OP_TO_STRING_CASE(v_cmpx_ngt_f64);
		OP_TO_STRING_CASE(v_cmpx_nle_f64);
		OP_TO_STRING_CASE(v_cmpx_neq_f64);
		OP_TO_STRING_CASE(v_cmpx_nlt_f64);
		OP_TO_STRING_CASE(v_cmpx_tru_f64);
		OP_TO_STRING_CASE(v_cmp_f_i32);
		OP_TO_STRING_CASE(v_cmp_lt_i32);
		OP_TO_STRING_CASE(v_cmp_eq_i32);
		OP_TO_STRING_CASE(v_cmp_le_i32);
		OP_TO_STRING_CASE(v_cmp_gt_i32);
		OP_TO_STRING_CASE(v_cmp_ne_i32);
		OP_TO_STRING_CASE(v_cmp_ge_i32);
		OP_TO_STRING_CASE(v_cmp_t_i32);
		OP_TO_STRING_CASE(v_cmp_class_f32);
		OP_TO_STRING_CASE(v_cmp_lt_i16);
		OP_TO_STRING_CASE(v_cmp_eq_i16);
		OP_TO_STRING_CASE(v_cmp_le_i16);
		OP_TO_STRING_CASE(v_cmp_gt_i16);
		OP_TO_STRING_CASE(v_cmp_ne_i16);
		OP_TO_STRING_CASE(v_cmp_ge_i16);
		OP_TO_STRING_CASE(v_cmp_class_f16);
		OP_TO_STRING_CASE(v_cmpx_f_i32);
		OP_TO_STRING_CASE(v_cmpx_lt_i32);
		OP_TO_STRING_CASE(v_cmpx_eq_i32);
		OP_TO_STRING_CASE(v_cmpx_le_i32);
		OP_TO_STRING_CASE(v_cmpx_gt_i32);
		OP_TO_STRING_CASE(v_cmpx_ne_i32);
		OP_TO_STRING_CASE(v_cmpx_ge_i32);
		OP_TO_STRING_CASE(v_cmpx_t_i32);
		OP_TO_STRING_CASE(v_cmpx_class_f32);
		OP_TO_STRING_CASE(v_cmpx_lt_i16);
		OP_TO_STRING_CASE(v_cmpx_eq_i16);
		OP_TO_STRING_CASE(v_cmpx_le_i16);
		OP_TO_STRING_CASE(v_cmpx_gt_i16);
		OP_TO_STRING_CASE(v_cmpx_ne_i16);
		OP_TO_STRING_CASE(v_cmpx_ge_i16);
		OP_TO_STRING_CASE(v_cmpx_class_f16);
		OP_TO_STRING_CASE(v_cmp_f_i64);
		OP_TO_STRING_CASE(v_cmp_lt_i64);
		OP_TO_STRING_CASE(v_cmp_eq_i64);
		OP_TO_STRING_CASE(v_cmp_le_i64);
		OP_TO_STRING_CASE(v_cmp_gt_i64);
		OP_TO_STRING_CASE(v_cmp_ne_i64);
		OP_TO_STRING_CASE(v_cmp_ge_i64);
		OP_TO_STRING_CASE(v_cmp_t_i64);
		OP_TO_STRING_CASE(v_cmp_class_f64);
		OP_TO_STRING_CASE(v_cmp_lt_u16);
		OP_TO_STRING_CASE(v_cmp_eq_u16);
		OP_TO_STRING_CASE(v_cmp_le_u16);
		OP_TO_STRING_CASE(v_cmp_gt_u16);
		OP_TO_STRING_CASE(v_cmp_ne_u16);
		OP_TO_STRING_CASE(v_cmp_ge_u16);
		OP_TO_STRING_CASE(v_cmpx_f_i64);
		OP_TO_STRING_CASE(v_cmpx_lt_i64);
		OP_TO_STRING_CASE(v_cmpx_eq_i64);
		OP_TO_STRING_CASE(v_cmpx_le_i64);
		OP_TO_STRING_CASE(v_cmpx_gt_i64);
		OP_TO_STRING_CASE(v_cmpx_ne_i64);
		OP_TO_STRING_CASE(v_cmpx_ge_i64);
		OP_TO_STRING_CASE(v_cmpx_t_i64);
		OP_TO_STRING_CASE(v_cmpx_class_f64);
		OP_TO_STRING_CASE(v_cmpx_lt_u16);
		OP_TO_STRING_CASE(v_cmpx_eq_u16);
		OP_TO_STRING_CASE(v_cmpx_le_u16);
		OP_TO_STRING_CASE(v_cmpx_gt_u16);
		OP_TO_STRING_CASE(v_cmpx_ne_u16);
		OP_TO_STRING_CASE(v_cmpx_ge_u16);
		OP_TO_STRING_CASE(v_cmp_f_u32);
		OP_TO_STRING_CASE(v_cmp_lt_u32);
		OP_TO_STRING_CASE(v_cmp_eq_u32);
		OP_TO_STRING_CASE(v_cmp_le_u32);
		OP_TO_STRING_CASE(v_cmp_gt_u32);
		OP_TO_STRING_CASE(v_cmp_ne_u32);
		OP_TO_STRING_CASE(v_cmp_ge_u32);
		OP_TO_STRING_CASE(v_cmp_t_u32);
		OP_TO_STRING_CASE(v_cmp_f_f16);
		OP_TO_STRING_CASE(v_cmp_lt_f16);
		OP_TO_STRING_CASE(v_cmp_eq_f16);
		OP_TO_STRING_CASE(v_cmp_le_f16);
		OP_TO_STRING_CASE(v_cmp_gt_f16);
		OP_TO_STRING_CASE(v_cmp_lg_f16);
		OP_TO_STRING_CASE(v_cmp_ge_f16);
		OP_TO_STRING_CASE(v_cmp_o_f16);
		OP_TO_STRING_CASE(v_cmpx_f_u32);
		OP_TO_STRING_CASE(v_cmpx_lt_u32);
		OP_TO_STRING_CASE(v_cmpx_eq_u32);
		OP_TO_STRING_CASE(v_cmpx_le_u32);
		OP_TO_STRING_CASE(v_cmpx_gt_u32);
		OP_TO_STRING_CASE(v_cmpx_ne_u32);
		OP_TO_STRING_CASE(v_cmpx_ge_u32);
		OP_TO_STRING_CASE(v_cmpx_t_u32);
		OP_TO_STRING_CASE(v_cmpx_f_f16);
		OP_TO_STRING_CASE(v_cmpx_lt_f16);
		OP_TO_STRING_CASE(v_cmpx_eq_f16);
		OP_TO_STRING_CASE(v_cmpx_le_f16);
		OP_TO_STRING_CASE(v_cmpx_gt_f16);
		OP_TO_STRING_CASE(v_cmpx_lg_f16);
		OP_TO_STRING_CASE(v_cmpx_ge_f16);
		OP_TO_STRING_CASE(v_cmpx_o_f16 );
		OP_TO_STRING_CASE(v_cmp_f_u64);
		OP_TO_STRING_CASE(v_cmp_lt_u64);
		OP_TO_STRING_CASE(v_cmp_eq_u64);
		OP_TO_STRING_CASE(v_cmp_le_u64);
		OP_TO_STRING_CASE(v_cmp_gt_u64);
		OP_TO_STRING_CASE(v_cmp_ne_u64);
		OP_TO_STRING_CASE(v_cmp_ge_u64);
		OP_TO_STRING_CASE(v_cmp_t_u64);
		OP_TO_STRING_CASE(v_cmp_u_f16);
		OP_TO_STRING_CASE(v_cmp_nge_f16);
		OP_TO_STRING_CASE(v_cmp_nlg_f16);
		OP_TO_STRING_CASE(v_cmp_ngt_f16);
		OP_TO_STRING_CASE(v_cmp_nle_f16);
		OP_TO_STRING_CASE(v_cmp_neq_f16);
		OP_TO_STRING_CASE(v_cmp_nlt_f16);
		OP_TO_STRING_CASE(v_cmp_tru_f16);
		OP_TO_STRING_CASE(v_cmpx_f_u64);
		OP_TO_STRING_CASE(v_cmpx_lt_u64);
		OP_TO_STRING_CASE(v_cmpx_eq_u64);
		OP_TO_STRING_CASE(v_cmpx_le_u64);
		OP_TO_STRING_CASE(v_cmpx_gt_u64);
		OP_TO_STRING_CASE(v_cmpx_ne_u64);
		OP_TO_STRING_CASE(v_cmpx_ge_u64);
		OP_TO_STRING_CASE(v_cmpx_t_u64);
		OP_TO_STRING_CASE(v_cmpx_u_f16);
		OP_TO_STRING_CASE(v_cmpx_nge_f16);
		OP_TO_STRING_CASE(v_cmpx_nlg_f16);
		OP_TO_STRING_CASE(v_cmpx_ngt_f16);
		OP_TO_STRING_CASE(v_cmpx_nle_f16);
		OP_TO_STRING_CASE(v_cmpx_neq_f16);
		OP_TO_STRING_CASE(v_cmpx_nlt_f16);
		OP_TO_STRING_CASE(v_cmpx_tru_f16);
	default:
		return "UNKNOWN - VOPC";
	}

#undef OP_TO_STRING_CASE
}

enum class EVOP3ABOps : uint16
{
	v_fma_legacy_f32 = 320,
	v_mad_f32 = 321,
	v_mad_i32_i24 = 322,
	v_mad_u32_u24 = 323,
	v_cubeid_f32 = 324,
	v_cubesc_f32 = 325,
	v_cubetc_f32 = 326,
	v_cubema_f32 = 327,
	v_bfe_u32 = 328,
	v_bfe_i32 = 329,
	v_bfi_b32 = 330,
	v_fma_f32 = 331,
	v_fma_f64 = 332,
	v_lerp_u8 = 333,
	v_alignbit_b32 = 334,
	v_alignbyte_b32 = 335,
	v_mullit_f32 = 336,
	v_min3_f32 = 337,
	v_min3_i32 = 338,
	v_min3_u32 = 339,
	v_max3_f32 = 340,
	v_max3_i32 = 341,
	v_max3_u32 = 342,
	v_med3_f32 = 343,
	v_med3_i32 = 344,
	v_med3_u32 = 345,
	v_sad_u8 = 346,
	v_sad_hi_u8 = 347,
	v_sad_u16 = 348,
	v_sad_u32 = 349,
	v_cvt_pk_u8_f32 = 350,
	v_div_fixup_f32 = 351,
	v_div_fixup_f64 = 352,
	v_add_f64 = 356,
	v_mul_f64 = 357,
	v_min_f64 = 358,
	v_max_f64 = 359,
	v_ldexp_f64 = 360,
	v_mul_lo_u32 = 361,
	v_mul_hi_u32 = 362,
	v_mul_hi_i32 = 364,
	v_div_scale_f32 = 365,
	v_div_scale_f64 = 366,
	v_div_fmas_f32 = 367,
	v_div_fmas_f64 = 368,
	v_msad_u8 = 369,
	v_qsad_pk_u16_u8 = 370,
	v_mqsad_pk_u16_u8 = 371,
	v_trig_preop_f64 = 372,
	v_mqsad_u32_u8 = 373,
	v_mad_u64_u32 = 374,
	v_mad_i64_i32 = 375,
	v_xor3_b32 = 376,
	v_lshlrev_b64 = 767,
	v_lshrrev_b64 = 768,
	v_ashrrev_i64 = 769,
	v_add_nc_u16 = 771,
	v_sub_nc_u16 = 772,
	v_mul_lo_u16 = 773,
	v_lshrrev_b16 = 775,
	v_ashrrev_i16 = 776,
	v_max_u16 = 777,
	v_max_i16 = 778,
	v_min_u16 = 779,
	v_min_i16 = 780,
	v_add_nc_i16 = 781,
	v_sub_nc_i16 = 782,
	v_add_co_u32 = 783,
	v_sub_co_u32 = 784,
	v_pack_b32_f16 = 785,
	v_cvt_pknorm_i16_f16 = 786,
	v_cvt_pknorm_u16_f16 = 787,
	v_lshlrev_b16 = 788,
	v_subrev_co_u32 = 793,
	v_mad_u16 = 832,
	v_interp_p1ll_f16 = 834,
	v_interp_p1lv_f16 = 835,
	v_perm_b32 = 836,
	v_xad_u32 = 837,
	v_lshl_add_u32 = 838,
	v_add_lshl_u32 = 839,
	v_fma_f16 = 843,
	v_min3_f16 = 849,
	v_min3_i16 = 850,
	v_min3_u16 = 851,
	v_max3_f16 = 852,
	v_max3_i16 = 853,
	v_max3_u16 = 854,
	v_med3_f16 = 855,
	v_med3_i16 = 856,
	v_med3_u16 = 857,
	v_interp_p2_f16 = 858,
	v_mad_i16 = 862,
	v_div_fixup_f16 = 863,
	v_readlane_b32 = 864,
	v_writelane_b32 = 865,
	v_ldexp_f32 = 866,
	v_bfm_b32 = 867,
	v_bcnt_u32_b32 = 868,
	v_mbcnt_lo_u32_b32 = 869,
	v_mbcnt_hi_u32_b32 = 870,
	v_cvt_pknorm_i16_f32 = 872,
	v_cvt_pknorm_u16_f32 = 873,
	v_cvt_pk_u16_u32 = 874,
	v_cvt_pk_i16_i32 = 875,
	v_add3_u32 = 877,
	v_lshl_or_b32 = 879,
	v_and_or_b32 = 881,
	v_or3_b32 = 882,
	v_mad_u32_u16 = 883,
	v_mad_i32_i16 = 885,
	v_sub_nc_i32 = 886,
	v_permlane16_b32 = 887,
	v_permlanex16_b32 = 888,
	v_add_nc_i32 = 895,
};

inline bool IsVOP3BEncoding(EVOP3ABOps Op)
{
	switch (Op)
	{
	case EVOP3ABOps::v_add_co_u32:
	case EVOP3ABOps::v_sub_co_u32:
	case EVOP3ABOps::v_subrev_co_u32:
	//case EVOP3ABOps::v_addc_co_u32: ???
	//case EVOP3ABOps::v_subb_co_u32: ???
	//case EVOP3ABOps::v_subbrev_co_u32: ???
	case EVOP3ABOps::v_div_scale_f32:
	case EVOP3ABOps::v_div_scale_f64:
	case EVOP3ABOps::v_mad_u64_u32:
	case EVOP3ABOps::v_mad_i64_i32:
		return true;
	default:
		return false;
	}
}

const char* ToString(EVOP3ABOps Op)
{
#define OP_TO_STRING_CASE(x) case EVOP3ABOps::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(v_fma_legacy_f32);
		OP_TO_STRING_CASE(v_mad_f32);
		OP_TO_STRING_CASE(v_mad_i32_i24);
		OP_TO_STRING_CASE(v_mad_u32_u24);
		OP_TO_STRING_CASE(v_cubeid_f32);
		OP_TO_STRING_CASE(v_cubesc_f32);
		OP_TO_STRING_CASE(v_cubetc_f32);
		OP_TO_STRING_CASE(v_cubema_f32);
		OP_TO_STRING_CASE(v_bfe_u32);
		OP_TO_STRING_CASE(v_bfe_i32);
		OP_TO_STRING_CASE(v_bfi_b32);
		OP_TO_STRING_CASE(v_fma_f32);
		OP_TO_STRING_CASE(v_fma_f64);
		OP_TO_STRING_CASE(v_lerp_u8);
		OP_TO_STRING_CASE(v_alignbit_b32);
		OP_TO_STRING_CASE(v_alignbyte_b32);
		OP_TO_STRING_CASE(v_mullit_f32);
		OP_TO_STRING_CASE(v_min3_f32);
		OP_TO_STRING_CASE(v_min3_i32);
		OP_TO_STRING_CASE(v_min3_u32);
		OP_TO_STRING_CASE(v_max3_f32);
		OP_TO_STRING_CASE(v_max3_i32);
		OP_TO_STRING_CASE(v_max3_u32);
		OP_TO_STRING_CASE(v_med3_f32);
		OP_TO_STRING_CASE(v_med3_i32);
		OP_TO_STRING_CASE(v_med3_u32);
		OP_TO_STRING_CASE(v_sad_u8);
		OP_TO_STRING_CASE(v_sad_hi_u8);
		OP_TO_STRING_CASE(v_sad_u16);
		OP_TO_STRING_CASE(v_sad_u32);
		OP_TO_STRING_CASE(v_cvt_pk_u8_f32);
		OP_TO_STRING_CASE(v_div_fixup_f32);
		OP_TO_STRING_CASE(v_div_fixup_f64);
		OP_TO_STRING_CASE(v_add_f64);
		OP_TO_STRING_CASE(v_mul_f64);
		OP_TO_STRING_CASE(v_min_f64);
		OP_TO_STRING_CASE(v_max_f64);
		OP_TO_STRING_CASE(v_ldexp_f64);
		OP_TO_STRING_CASE(v_mul_lo_u32);
		OP_TO_STRING_CASE(v_mul_hi_u32);
		OP_TO_STRING_CASE(v_mul_hi_i32);
		OP_TO_STRING_CASE(v_div_scale_f32);
		OP_TO_STRING_CASE(v_div_scale_f64);
		OP_TO_STRING_CASE(v_div_fmas_f32);
		OP_TO_STRING_CASE(v_div_fmas_f64);
		OP_TO_STRING_CASE(v_msad_u8);
		OP_TO_STRING_CASE(v_qsad_pk_u16_u8);
		OP_TO_STRING_CASE(v_mqsad_pk_u16_u8);
		OP_TO_STRING_CASE(v_trig_preop_f64);
		OP_TO_STRING_CASE(v_mqsad_u32_u8);
		OP_TO_STRING_CASE(v_mad_u64_u32);
		OP_TO_STRING_CASE(v_mad_i64_i32);
		OP_TO_STRING_CASE(v_xor3_b32);
		OP_TO_STRING_CASE(v_lshlrev_b64);
		OP_TO_STRING_CASE(v_lshrrev_b64);
		OP_TO_STRING_CASE(v_ashrrev_i64);
		OP_TO_STRING_CASE(v_add_nc_u16);
		OP_TO_STRING_CASE(v_sub_nc_u16);
		OP_TO_STRING_CASE(v_mul_lo_u16);
		OP_TO_STRING_CASE(v_lshrrev_b16);
		OP_TO_STRING_CASE(v_ashrrev_i16);
		OP_TO_STRING_CASE(v_max_u16);
		OP_TO_STRING_CASE(v_max_i16);
		OP_TO_STRING_CASE(v_min_u16);
		OP_TO_STRING_CASE(v_min_i16);
		OP_TO_STRING_CASE(v_add_nc_i16);
		OP_TO_STRING_CASE(v_sub_nc_i16);
		OP_TO_STRING_CASE(v_add_co_u32);
		OP_TO_STRING_CASE(v_sub_co_u32);
		OP_TO_STRING_CASE(v_pack_b32_f16);
		OP_TO_STRING_CASE(v_cvt_pknorm_i16_f16);
		OP_TO_STRING_CASE(v_cvt_pknorm_u16_f16);
		OP_TO_STRING_CASE(v_lshlrev_b16);
		OP_TO_STRING_CASE(v_subrev_co_u32);
		OP_TO_STRING_CASE(v_mad_u16);
		OP_TO_STRING_CASE(v_interp_p1ll_f16);
		OP_TO_STRING_CASE(v_interp_p1lv_f16);
		OP_TO_STRING_CASE(v_perm_b32);
		OP_TO_STRING_CASE(v_xad_u32);
		OP_TO_STRING_CASE(v_lshl_add_u32);
		OP_TO_STRING_CASE(v_add_lshl_u32);
		OP_TO_STRING_CASE(v_fma_f16);
		OP_TO_STRING_CASE(v_min3_f16);
		OP_TO_STRING_CASE(v_min3_i16);
		OP_TO_STRING_CASE(v_min3_u16);
		OP_TO_STRING_CASE(v_max3_f16);
		OP_TO_STRING_CASE(v_max3_i16);
		OP_TO_STRING_CASE(v_max3_u16);
		OP_TO_STRING_CASE(v_med3_f16);
		OP_TO_STRING_CASE(v_med3_i16);
		OP_TO_STRING_CASE(v_med3_u16);
		OP_TO_STRING_CASE(v_interp_p2_f16);
		OP_TO_STRING_CASE(v_mad_i16);
		OP_TO_STRING_CASE(v_div_fixup_f16);
		OP_TO_STRING_CASE(v_readlane_b32);
		OP_TO_STRING_CASE(v_writelane_b32);
		OP_TO_STRING_CASE(v_ldexp_f32);
		OP_TO_STRING_CASE(v_bfm_b32);
		OP_TO_STRING_CASE(v_bcnt_u32_b32);
		OP_TO_STRING_CASE(v_mbcnt_lo_u32_b32);
		OP_TO_STRING_CASE(v_mbcnt_hi_u32_b32);
		OP_TO_STRING_CASE(v_cvt_pknorm_i16_f32);
		OP_TO_STRING_CASE(v_cvt_pknorm_u16_f32);
		OP_TO_STRING_CASE(v_cvt_pk_u16_u32);
		OP_TO_STRING_CASE(v_cvt_pk_i16_i32);
		OP_TO_STRING_CASE(v_add3_u32);
		OP_TO_STRING_CASE(v_lshl_or_b32);
		OP_TO_STRING_CASE(v_and_or_b32);
		OP_TO_STRING_CASE(v_or3_b32);
		OP_TO_STRING_CASE(v_mad_u32_u16);
		OP_TO_STRING_CASE(v_mad_i32_i16);
		OP_TO_STRING_CASE(v_sub_nc_i32);
		OP_TO_STRING_CASE(v_permlane16_b32);
		OP_TO_STRING_CASE(v_permlanex16_b32);
		OP_TO_STRING_CASE(v_add_nc_i32);
	default:
		return "UNKNOWN - VOP3 A/B";
	}

#undef OP_TO_STRING_CASE
}

enum class EVINTERPOps : uint16
{
	v_interp_p1_f32 = 0,
	v_interp_p2_f32 = 1,
	v_interp_mov_f32 = 2,
};

const char* ToString(EVINTERPOps Op)
{
#define OP_TO_STRING_CASE(x) case EVINTERPOps::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(v_interp_p1_f32);
		OP_TO_STRING_CASE(v_interp_p2_f32);
		OP_TO_STRING_CASE(v_interp_mov_f32);
	default:
		return "UNKNOWN - VINTERP";
	}

#undef OP_TO_STRING_CASE
}

enum class EMIMGOps : uint16
{
	image_load = 0,
	image_load_mip = 1,
	image_load_pck = 2,
	image_load_pck_sgn = 3,
	image_load_mip_pck = 4,
	image_load_mip_pck_sgn = 5,
	image_store = 8,
	image_store_mip = 9,
	image_store_pck = 10,
	image_store_mip_pck = 11,
	image_get_resinfo = 14,
	image_atomic_swap = 15,
	image_atomic_cmpswap = 16,
	image_atomic_add = 17,
	image_atomic_sub = 18,
	image_atomic_smin = 20,
	image_atomic_umin = 21,
	image_atomic_smax = 22,
	image_atomic_umax = 23,
	image_atomic_and = 24,
	image_atomic_or = 25,
	image_atomic_xor = 26,
	image_atomic_inc = 27,
	image_atomic_dec = 28,
	image_atomic_fcmpswap = 29,
	image_atomic_fmin = 30,
	image_atomic_fmax = 31,
	image_sample = 32,
	image_sample_cl = 33,
	image_sample_d = 34,
	image_sample_d_cl = 35,
	image_sample_l = 36,
	image_sample_b = 37,
	image_sample_b_cl = 38,
	image_sample_lz = 39,
	image_sample_c = 40,
	image_sample_c_cl = 41,
	image_sample_c_d = 42,
	image_sample_c_d_cl = 43,
	image_sample_c_l = 44,
	image_sample_c_b = 45,
	image_sample_c_b_cl = 46,
	image_sample_c_lz = 47,
	image_sample_o = 48,
	image_sample_cl_o = 49,
	image_sample_d_o = 50,
	image_sample_d_cl_o = 51,
	image_sample_l_o = 52,
	image_sample_b_o = 53,
	image_sample_b_cl_o = 54,
	image_sample_lz_o = 55,
	image_sample_c_o = 56,
	image_sample_c_cl_o = 57,
	image_sample_c_do_o = 58,
	image_sample_c_d_cl_o = 59,
	image_sample_c_l_o = 60,
	image_sample_c_b_o = 61,
	image_sample_c_b_cl_o = 62,
	image_sample_c_lz_o = 63,
	image_gather4 = 64,
	image_gather4_cl = 65,
	image_gather4_l = 68,
	image_gather4_b = 69,
	image_gather4_b_cl = 70,
	image_gather4_lz = 71,
	image_gather4_c = 72,
	image_gather4_c_cl = 73,
	image_gather4_c_l = 76,
	image_gather4_c_b = 77,
	image_gather4_c_b_cl = 78,
	image_gather4_c_lz = 79,
	image_gather4_o = 80,
	image_gather4_cl_o = 81,
	image_gather4_l_o = 84,
	image_gather4_b_o = 85,
	image_gather4_b_cl_o = 86,
	image_gather4_lz_o = 87,
	image_gather4_c_o = 88,
	image_gather4_c_cl_o = 89,
	image_gather4_c_l_o = 92,
	image_gather4_c_b_o = 93,
	image_gather4_c_b_cl_o = 94,
	image_gather4_c_lz_o = 95,
	image_get_lod = 96,
	image_gather4h = 97,
	image_gather4h_pck = 98,
	image_gather8h_pck = 99,
	image_sample_d_g16 = 162,
	image_sample_d_cl_g16 = 163,
	image_sample_c_d_g16 = 170,
	image_sample_c_d_cl_g16 = 171,
	image_sample_d_o_g16 = 178,
	image_sample_d_cl_o_g16 = 179,
	image_sample_c_d_o_g16 = 186,
	image_sample_c_d_cl_o_g16 = 187,
};

const char* ToString(EMIMGOps Op)
{
#define OP_TO_STRING_CASE(x) case EMIMGOps::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(image_load);
		OP_TO_STRING_CASE(image_load_mip);
		OP_TO_STRING_CASE(image_load_pck);
		OP_TO_STRING_CASE(image_load_pck_sgn);
		OP_TO_STRING_CASE(image_load_mip_pck);
		OP_TO_STRING_CASE(image_load_mip_pck_sgn);
		OP_TO_STRING_CASE(image_store);
		OP_TO_STRING_CASE(image_store_mip);
		OP_TO_STRING_CASE(image_store_pck);
		OP_TO_STRING_CASE(image_store_mip_pck);
		OP_TO_STRING_CASE(image_get_resinfo);
		OP_TO_STRING_CASE(image_atomic_swap);
		OP_TO_STRING_CASE(image_atomic_cmpswap);
		OP_TO_STRING_CASE(image_atomic_add);
		OP_TO_STRING_CASE(image_atomic_sub);
		OP_TO_STRING_CASE(image_atomic_smin);
		OP_TO_STRING_CASE(image_atomic_umin);
		OP_TO_STRING_CASE(image_atomic_smax);
		OP_TO_STRING_CASE(image_atomic_umax);
		OP_TO_STRING_CASE(image_atomic_and);
		OP_TO_STRING_CASE(image_atomic_or);
		OP_TO_STRING_CASE(image_atomic_xor);
		OP_TO_STRING_CASE(image_atomic_inc);
		OP_TO_STRING_CASE(image_atomic_dec);
		OP_TO_STRING_CASE(image_atomic_fcmpswap);
		OP_TO_STRING_CASE(image_atomic_fmin);
		OP_TO_STRING_CASE(image_atomic_fmax);
		OP_TO_STRING_CASE(image_sample);
		OP_TO_STRING_CASE(image_sample_cl);
		OP_TO_STRING_CASE(image_sample_d);
		OP_TO_STRING_CASE(image_sample_d_cl);
		OP_TO_STRING_CASE(image_sample_l);
		OP_TO_STRING_CASE(image_sample_b);
		OP_TO_STRING_CASE(image_sample_b_cl);
		OP_TO_STRING_CASE(image_sample_lz);
		OP_TO_STRING_CASE(image_sample_c);
		OP_TO_STRING_CASE(image_sample_c_cl);
		OP_TO_STRING_CASE(image_sample_c_d);
		OP_TO_STRING_CASE(image_sample_c_d_cl);
		OP_TO_STRING_CASE(image_sample_c_l);
		OP_TO_STRING_CASE(image_sample_c_b);
		OP_TO_STRING_CASE(image_sample_c_b_cl);
		OP_TO_STRING_CASE(image_sample_c_lz);
		OP_TO_STRING_CASE(image_sample_o);
		OP_TO_STRING_CASE(image_sample_cl_o);
		OP_TO_STRING_CASE(image_sample_d_o);
		OP_TO_STRING_CASE(image_sample_d_cl_o);
		OP_TO_STRING_CASE(image_sample_l_o);
		OP_TO_STRING_CASE(image_sample_b_o);
		OP_TO_STRING_CASE(image_sample_b_cl_o);
		OP_TO_STRING_CASE(image_sample_lz_o);
		OP_TO_STRING_CASE(image_sample_c_o);
		OP_TO_STRING_CASE(image_sample_c_cl_o);
		OP_TO_STRING_CASE(image_sample_c_do_o);
		OP_TO_STRING_CASE(image_sample_c_d_cl_o);
		OP_TO_STRING_CASE(image_sample_c_l_o);
		OP_TO_STRING_CASE(image_sample_c_b_o);
		OP_TO_STRING_CASE(image_sample_c_b_cl_o);
		OP_TO_STRING_CASE(image_sample_c_lz_o);
		OP_TO_STRING_CASE(image_gather4);
		OP_TO_STRING_CASE(image_gather4_cl);
		OP_TO_STRING_CASE(image_gather4_l);
		OP_TO_STRING_CASE(image_gather4_b);
		OP_TO_STRING_CASE(image_gather4_b_cl);
		OP_TO_STRING_CASE(image_gather4_lz);
		OP_TO_STRING_CASE(image_gather4_c);
		OP_TO_STRING_CASE(image_gather4_c_cl);
		OP_TO_STRING_CASE(image_gather4_c_l);
		OP_TO_STRING_CASE(image_gather4_c_b);
		OP_TO_STRING_CASE(image_gather4_c_b_cl);
		OP_TO_STRING_CASE(image_gather4_c_lz);
		OP_TO_STRING_CASE(image_gather4_o);
		OP_TO_STRING_CASE(image_gather4_cl_o);
		OP_TO_STRING_CASE(image_gather4_l_o);
		OP_TO_STRING_CASE(image_gather4_b_o);
		OP_TO_STRING_CASE(image_gather4_b_cl_o);
		OP_TO_STRING_CASE(image_gather4_lz_o);
		OP_TO_STRING_CASE(image_gather4_c_o);
		OP_TO_STRING_CASE(image_gather4_c_cl_o);
		OP_TO_STRING_CASE(image_gather4_c_l_o);
		OP_TO_STRING_CASE(image_gather4_c_b_o);
		OP_TO_STRING_CASE(image_gather4_c_b_cl_o);
		OP_TO_STRING_CASE(image_gather4_c_lz_o);
		OP_TO_STRING_CASE(image_get_lod);
		OP_TO_STRING_CASE(image_gather4h);
		OP_TO_STRING_CASE(image_gather4h_pck);
		OP_TO_STRING_CASE(image_gather8h_pck);
		OP_TO_STRING_CASE(image_sample_d_g16);
		OP_TO_STRING_CASE(image_sample_d_cl_g16);
		OP_TO_STRING_CASE(image_sample_c_d_g16);
		OP_TO_STRING_CASE(image_sample_c_d_cl_g16);
		OP_TO_STRING_CASE(image_sample_d_o_g16);
		OP_TO_STRING_CASE(image_sample_d_cl_o_g16);
		OP_TO_STRING_CASE(image_sample_c_d_o_g16);
		OP_TO_STRING_CASE(image_sample_c_d_cl_o_g16);
	default:
		return "UNKNOWN - MIMG";
	}

#undef OP_TO_STRING_CASE
}

enum class EMUBUFOps : uint16
{
	buffer_load_format_x = 0,
	buffer_load_format_xy = 1,
	buffer_load_format_xyz = 2,
	buffer_load_format_xyzw = 3,
	buffer_store_format_x = 4,
	buffer_store_format_xy = 5,
	buffer_store_format_xyz = 6,
	buffer_store_format_xyzw = 7,
	buffer_load_ubyte = 8,
	buffer_load_sbyte = 9,
	buffer_load_ushort = 10,
	buffer_load_sshort = 11,
	buffer_load_dword = 12,
	buffer_load_dwordx2 = 13,
	buffer_load_dwordx4 = 14,
	buffer_load_dwordx3 = 15,
	buffer_store_byte = 24,
	buffer_store_byte_d16_hi = 25,
	buffer_store_short = 26,
	buffer_store_short_d16_hi = 27,
	buffer_store_dword = 28,
	buffer_store_dwordx2 = 29,
	buffer_store_dwordx4 = 30,
	buffer_store_dwordx3 = 31,
	buffer_load_ubyte_d16 = 32,
	buffer_load_ubyte_d16_hi = 33,
	buffer_load_sbyte_d16 = 34,
	buffer_load_sbyte_d16_hi = 35,
	buffer_load_short_d16 = 36,
	buffer_load_short_d16_hi = 37,
	buffer_load_format_d16_hi_x = 38,
	buffer_store_format_d16_hi_x = 39,
	buffer_atomic_swap = 48,
	buffer_atomic_cmpswap = 49,
	buffer_atomic_add = 50,
	buffer_atomic_sub = 51,
	buffer_atomic_csub = 52,
	buffer_atomic_smin = 53,
	buffer_atomic_umin = 54,
	buffer_atomic_smax = 55,
	buffer_atomic_umax = 56,
	buffer_atomic_and = 57,
	buffer_atomic_or = 58,
	buffer_atomic_xor = 59,
	buffer_atomic_inc = 60,
	buffer_atomic_dec = 61,
	buffer_atomic_fcmpswap = 62,
	buffer_atomic_fmin = 63,
	buffer_atomic_fmax = 64,
	buffer_atomic_swap_x2 = 80,
	buffer_atomic_cmpswap_x2 = 81,
	buffer_atomic_add_x2 = 82,
	buffer_atomic_sub_x2 = 83,
	buffer_atomic_smin_x2 = 85,
	buffer_atomic_umin_x2 = 86,
	buffer_atomic_smax_x2 = 87,
	buffer_atomic_umax_x2 = 88,
	buffer_atomic_and_x2 = 89,
	buffer_atomic_or_x2 = 90,
	buffer_atomic_xor_x2 = 91,
	buffer_atomic_inc_x2 = 92,
	buffer_atomic_dec_x2 = 93,
	buffer_atomic_fcmpswap_x2 = 94,
	buffer_atomic_fmin_x2 = 95,
	buffer_atomic_fmax_x2 = 96,
	buffer_gl0_inv = 113,
	buffer_gl1_inv = 114,
	buffer_load_format_d16_x = 128,
	buffer_load_format_d16_xy = 129,
	buffer_load_format_d16_xyz = 130,
	buffer_load_format_d16_xyzw = 131,
	buffer_store_format_d16_x = 132,
	buffer_store_format_d16_xy = 133,
	buffer_store_format_d16_xyz = 134,
	buffer_store_format_d16_xyzw = 135,
};

const char* ToString(EMUBUFOps Op)
{
#define OP_TO_STRING_CASE(x) case EMUBUFOps::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(buffer_load_format_x);
		OP_TO_STRING_CASE(buffer_load_format_xy);
		OP_TO_STRING_CASE(buffer_load_format_xyz);
		OP_TO_STRING_CASE(buffer_load_format_xyzw);
		OP_TO_STRING_CASE(buffer_store_format_x);
		OP_TO_STRING_CASE(buffer_store_format_xy);
		OP_TO_STRING_CASE(buffer_store_format_xyz);
		OP_TO_STRING_CASE(buffer_store_format_xyzw);
		OP_TO_STRING_CASE(buffer_load_ubyte);
		OP_TO_STRING_CASE(buffer_load_sbyte);
		OP_TO_STRING_CASE(buffer_load_ushort);
		OP_TO_STRING_CASE(buffer_load_sshort);
		OP_TO_STRING_CASE(buffer_load_dword);
		OP_TO_STRING_CASE(buffer_load_dwordx2);
		OP_TO_STRING_CASE(buffer_load_dwordx4);
		OP_TO_STRING_CASE(buffer_load_dwordx3);
		OP_TO_STRING_CASE(buffer_store_byte);
		OP_TO_STRING_CASE(buffer_store_byte_d16_hi);
		OP_TO_STRING_CASE(buffer_store_short);
		OP_TO_STRING_CASE(buffer_store_short_d16_hi);
		OP_TO_STRING_CASE(buffer_store_dword);
		OP_TO_STRING_CASE(buffer_store_dwordx2);
		OP_TO_STRING_CASE(buffer_store_dwordx4);
		OP_TO_STRING_CASE(buffer_store_dwordx3);
		OP_TO_STRING_CASE(buffer_load_ubyte_d16);
		OP_TO_STRING_CASE(buffer_load_ubyte_d16_hi);
		OP_TO_STRING_CASE(buffer_load_sbyte_d16);
		OP_TO_STRING_CASE(buffer_load_sbyte_d16_hi);
		OP_TO_STRING_CASE(buffer_load_short_d16);
		OP_TO_STRING_CASE(buffer_load_short_d16_hi);
		OP_TO_STRING_CASE(buffer_load_format_d16_hi_x);
		OP_TO_STRING_CASE(buffer_store_format_d16_hi_x);
		OP_TO_STRING_CASE(buffer_atomic_swap);
		OP_TO_STRING_CASE(buffer_atomic_cmpswap);
		OP_TO_STRING_CASE(buffer_atomic_add);
		OP_TO_STRING_CASE(buffer_atomic_sub);
		OP_TO_STRING_CASE(buffer_atomic_csub);
		OP_TO_STRING_CASE(buffer_atomic_smin);
		OP_TO_STRING_CASE(buffer_atomic_umin);
		OP_TO_STRING_CASE(buffer_atomic_smax);
		OP_TO_STRING_CASE(buffer_atomic_umax);
		OP_TO_STRING_CASE(buffer_atomic_and);
		OP_TO_STRING_CASE(buffer_atomic_or);
		OP_TO_STRING_CASE(buffer_atomic_xor);
		OP_TO_STRING_CASE(buffer_atomic_inc);
		OP_TO_STRING_CASE(buffer_atomic_dec);
		OP_TO_STRING_CASE(buffer_atomic_fcmpswap);
		OP_TO_STRING_CASE(buffer_atomic_fmin);
		OP_TO_STRING_CASE(buffer_atomic_fmax);
		OP_TO_STRING_CASE(buffer_atomic_swap_x2);
		OP_TO_STRING_CASE(buffer_atomic_cmpswap_x2);
		OP_TO_STRING_CASE(buffer_atomic_add_x2);
		OP_TO_STRING_CASE(buffer_atomic_sub_x2);
		OP_TO_STRING_CASE(buffer_atomic_smin_x2);
		OP_TO_STRING_CASE(buffer_atomic_umin_x2);
		OP_TO_STRING_CASE(buffer_atomic_smax_x2);
		OP_TO_STRING_CASE(buffer_atomic_umax_x2);
		OP_TO_STRING_CASE(buffer_atomic_and_x2);
		OP_TO_STRING_CASE(buffer_atomic_or_x2);
		OP_TO_STRING_CASE(buffer_atomic_xor_x2);
		OP_TO_STRING_CASE(buffer_atomic_inc_x2);
		OP_TO_STRING_CASE(buffer_atomic_dec_x2);
		OP_TO_STRING_CASE(buffer_atomic_fcmpswap_x2);
		OP_TO_STRING_CASE(buffer_atomic_fmin_x2);
		OP_TO_STRING_CASE(buffer_atomic_fmax_x2);
		OP_TO_STRING_CASE(buffer_gl0_inv);
		OP_TO_STRING_CASE(buffer_gl1_inv);
		OP_TO_STRING_CASE(buffer_load_format_d16_x);
		OP_TO_STRING_CASE(buffer_load_format_d16_xy);
		OP_TO_STRING_CASE(buffer_load_format_d16_xyz);
		OP_TO_STRING_CASE(buffer_load_format_d16_xyzw);
		OP_TO_STRING_CASE(buffer_store_format_d16_x);
		OP_TO_STRING_CASE(buffer_store_format_d16_xy);
		OP_TO_STRING_CASE(buffer_store_format_d16_xyz);
		OP_TO_STRING_CASE(buffer_store_format_d16_xyzw);
	default:
		return "UNKNOWN - MUBUF";
	}

#undef OP_TO_STRING_CASE
}

enum class EMTBUFOps : uint16
{
	tbuffer_load_format_x = 0,
	tbuffer_load_format_xy = 1,
	tbuffer_load_format_xyz = 2,
	tbuffer_load_format_xyzw = 3,
	tbuffer_store_format_x = 4,
	tbuffer_store_format_xy = 5,
	tbuffer_store_format_xyz = 6,
	tbuffer_store_format_xyzw = 7,
	tbuffer_load_format_d16_x = 8,
	tbuffer_load_format_d16_xy = 9,
	tbuffer_load_format_d16_xyz = 10,
	tbuffer_load_format_d16_xyzw = 11,
	tbuffer_store_format_d16_x = 12,
	tbuffer_store_format_d16_xy = 13,
	tbuffer_store_format_d16_xyz = 14,
	tbuffer_store_format_d16_xyzw = 15,
};

const char* ToString(EMTBUFOps Op)
{
#define OP_TO_STRING_CASE(x) case EMTBUFOps::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(tbuffer_load_format_x);
		OP_TO_STRING_CASE(tbuffer_load_format_xy);
		OP_TO_STRING_CASE(tbuffer_load_format_xyz);
		OP_TO_STRING_CASE(tbuffer_load_format_xyzw);
		OP_TO_STRING_CASE(tbuffer_store_format_x);
		OP_TO_STRING_CASE(tbuffer_store_format_xy);
		OP_TO_STRING_CASE(tbuffer_store_format_xyz);
		OP_TO_STRING_CASE(tbuffer_store_format_xyzw);
		OP_TO_STRING_CASE(tbuffer_load_format_d16_x);
		OP_TO_STRING_CASE(tbuffer_load_format_d16_xy);
		OP_TO_STRING_CASE(tbuffer_load_format_d16_xyz);
		OP_TO_STRING_CASE(tbuffer_load_format_d16_xyzw);
		OP_TO_STRING_CASE(tbuffer_store_format_d16_x);
		OP_TO_STRING_CASE(tbuffer_store_format_d16_xy);
		OP_TO_STRING_CASE(tbuffer_store_format_d16_xyz);
		OP_TO_STRING_CASE(tbuffer_store_format_d16_xyzw);
	default:
		return "UNKNOWN - MTBUF";
	}

#undef OP_TO_STRING_CASE
}

enum class EFLATOps : uint16
{
	flat_load_ubyte = 8,
	flat_load_sbyte = 9,
	flat_load_ushort = 10,
	flat_load_sshort = 11,
	flat_load_dword = 12,
	flat_load_dwordx2 = 13,
	flat_load_dwordx4 = 14,
	flat_load_dwordx3 = 15,
	flat_store_byte = 24,
	flat_store_byte_d16_hi = 25,
	flat_store_short = 26,
	flat_store_short_d16_hi = 27,
	flat_store_dword = 28,
	flat_store_dwordx2 = 29,
	flat_store_dwordx4 = 30,
	flat_store_dwordx3 = 31,
	flat_load_ubyte_d16 = 32,
	flat_load_ubyte_d16_hi = 33,
	flat_load_sbyte_d16 = 34,
	flat_load_sbyte_d16_hi = 35,
	flat_load_short_d16 = 36,
	flat_load_short_d16_hi = 37,
	flat_atomic_swap = 48,
	flat_atomic_cmpswap = 49,
	flat_atomic_add = 50,
	flat_atomic_sub = 51,
	flat_atomic_smin = 53,
	flat_atomic_umin = 54,
	flat_atomic_smax = 55,
	flat_atomic_umax = 56,
	flat_atomic_and = 57,
	flat_atomic_or = 58,
	flat_atomic_xor = 59,
	flat_atomic_inc = 60,
	flat_atomic_dec = 61,
	flat_atomic_fcmpswap = 62,
	flat_atomic_fmin = 63,
	flat_atomic_fmax = 64,
	flat_atomic_swap_x2 = 80,
	flat_atomic_cmpswap_x2 = 81,
	flat_atomic_add_x2 = 82,
	flat_atomic_sub_x2 = 83,
	flat_atomic_smin_x2 = 85,
	flat_atomic_umin_x2 = 86,
	flat_atomic_smax_x2 = 87,
	flat_atomic_umax_x2 = 88,
	flat_atomic_and_x2 = 89,
	flat_atomic_or_x2 = 90,
	flat_atomic_xor_x2 = 91,
	flat_atomic_inc_x2 = 92,
	flat_atomic_dec_x2 = 93,
	flat_atomic_fcmpswap_x2 = 94,
	flat_atomic_fmin_x2 = 95,
	flat_atomic_fmax_x2 = 96,
};

const char* ToString(EFLATOps Op)
{
#define OP_TO_STRING_CASE(x) case EFLATOps::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(flat_load_ubyte);
		OP_TO_STRING_CASE(flat_load_sbyte);
		OP_TO_STRING_CASE(flat_load_ushort);
		OP_TO_STRING_CASE(flat_load_sshort);
		OP_TO_STRING_CASE(flat_load_dword);
		OP_TO_STRING_CASE(flat_load_dwordx2);
		OP_TO_STRING_CASE(flat_load_dwordx4);
		OP_TO_STRING_CASE(flat_load_dwordx3);
		OP_TO_STRING_CASE(flat_store_byte);
		OP_TO_STRING_CASE(flat_store_byte_d16_hi);
		OP_TO_STRING_CASE(flat_store_short);
		OP_TO_STRING_CASE(flat_store_short_d16_hi);
		OP_TO_STRING_CASE(flat_store_dword);
		OP_TO_STRING_CASE(flat_store_dwordx2);
		OP_TO_STRING_CASE(flat_store_dwordx4);
		OP_TO_STRING_CASE(flat_store_dwordx3);
		OP_TO_STRING_CASE(flat_load_ubyte_d16);
		OP_TO_STRING_CASE(flat_load_ubyte_d16_hi);
		OP_TO_STRING_CASE(flat_load_sbyte_d16);
		OP_TO_STRING_CASE(flat_load_sbyte_d16_hi);
		OP_TO_STRING_CASE(flat_load_short_d16);
		OP_TO_STRING_CASE(flat_load_short_d16_hi);
		OP_TO_STRING_CASE(flat_atomic_swap);
		OP_TO_STRING_CASE(flat_atomic_cmpswap);
		OP_TO_STRING_CASE(flat_atomic_add);
		OP_TO_STRING_CASE(flat_atomic_sub);
		OP_TO_STRING_CASE(flat_atomic_smin);
		OP_TO_STRING_CASE(flat_atomic_umin);
		OP_TO_STRING_CASE(flat_atomic_smax);
		OP_TO_STRING_CASE(flat_atomic_umax);
		OP_TO_STRING_CASE(flat_atomic_and);
		OP_TO_STRING_CASE(flat_atomic_or);
		OP_TO_STRING_CASE(flat_atomic_xor);
		OP_TO_STRING_CASE(flat_atomic_inc);
		OP_TO_STRING_CASE(flat_atomic_dec);
		OP_TO_STRING_CASE(flat_atomic_fcmpswap);
		OP_TO_STRING_CASE(flat_atomic_fmin);
		OP_TO_STRING_CASE(flat_atomic_fmax);
		OP_TO_STRING_CASE(flat_atomic_swap_x2);
		OP_TO_STRING_CASE(flat_atomic_cmpswap_x2);
		OP_TO_STRING_CASE(flat_atomic_add_x2);
		OP_TO_STRING_CASE(flat_atomic_sub_x2);
		OP_TO_STRING_CASE(flat_atomic_smin_x2);
		OP_TO_STRING_CASE(flat_atomic_umin_x2);
		OP_TO_STRING_CASE(flat_atomic_smax_x2);
		OP_TO_STRING_CASE(flat_atomic_umax_x2);
		OP_TO_STRING_CASE(flat_atomic_and_x2);
		OP_TO_STRING_CASE(flat_atomic_or_x2);
		OP_TO_STRING_CASE(flat_atomic_xor_x2);
		OP_TO_STRING_CASE(flat_atomic_inc_x2);
		OP_TO_STRING_CASE(flat_atomic_dec_x2);
		OP_TO_STRING_CASE(flat_atomic_fcmpswap_x2);
		OP_TO_STRING_CASE(flat_atomic_fmin_x2);
		OP_TO_STRING_CASE(flat_atomic_fmax_x2);
	default:
		return "UNKNOWN - FLAT";
	}

#undef OP_TO_STRING_CASE
}

enum class ESCRATCHOps : uint16
{
	scratch_load_ubyte = 8,
	scratch_load_sbyte = 9,
	scratch_load_ushort = 10,
	scratch_load_sshort = 11,
	scratch_load_dword = 12,
	scratch_load_dwordx2 = 13,
	scratch_load_dwordx4 = 14,
	scratch_load_dwordx3 = 15,
	scratch_store_byte = 24,
	scratch_store_byte_d16_hi = 25,
	scratch_store_short = 26,
	scratch_store_short_d16_hi = 27,
	scratch_store_dword = 28,
	scratch_store_dwordx2 = 29,
	scratch_store_dwordx4 = 30,
	scratch_store_dwordx3 = 31,
	scratch_load_ubyte_d16 = 32,
	scratch_load_ubyte_d16_hi = 33,
	scratch_load_sbyte_d16 = 34,
	scratch_load_sbyte_d16_hi = 35,
	scratch_load_short_d16 = 36,
	scratch_load_short_d16_hi = 37,
};

const char* ToString(ESCRATCHOps Op)
{
#define OP_TO_STRING_CASE(x) case ESCRATCHOps::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(scratch_load_ubyte);
		OP_TO_STRING_CASE(scratch_load_sbyte);
		OP_TO_STRING_CASE(scratch_load_ushort);
		OP_TO_STRING_CASE(scratch_load_sshort);
		OP_TO_STRING_CASE(scratch_load_dword);
		OP_TO_STRING_CASE(scratch_load_dwordx2);
		OP_TO_STRING_CASE(scratch_load_dwordx4);
		OP_TO_STRING_CASE(scratch_load_dwordx3);
		OP_TO_STRING_CASE(scratch_store_byte);
		OP_TO_STRING_CASE(scratch_store_byte_d16_hi);
		OP_TO_STRING_CASE(scratch_store_short);
		OP_TO_STRING_CASE(scratch_store_short_d16_hi);
		OP_TO_STRING_CASE(scratch_store_dword);
		OP_TO_STRING_CASE(scratch_store_dwordx2);
		OP_TO_STRING_CASE(scratch_store_dwordx4);
		OP_TO_STRING_CASE(scratch_store_dwordx3);
		OP_TO_STRING_CASE(scratch_load_ubyte_d16);
		OP_TO_STRING_CASE(scratch_load_ubyte_d16_hi);
		OP_TO_STRING_CASE(scratch_load_sbyte_d16);
		OP_TO_STRING_CASE(scratch_load_sbyte_d16_hi);
		OP_TO_STRING_CASE(scratch_load_short_d16);
		OP_TO_STRING_CASE(scratch_load_short_d16_hi);
	default:
		return "UNKNOWN - SCRATCH";
	}

#undef OP_TO_STRING_CASE
}

enum class ELDSGDSOps : uint16
{
	ds_add_u32 = 0,
	ds_sub_u32 = 1,
	ds_rsub_u32 = 2,
	ds_inc_u32 = 3,
	ds_dec_u32 = 4,
	ds_min_i32 = 5,
	ds_max_i32 = 6,
	ds_min_u32 = 7,
	ds_max_u32 = 8,
	ds_and_b32 = 9,
	ds_or_b32 = 10,
	ds_xor_b32 = 11,
	ds_mskor_b32 = 12,
	ds_write_b32 = 13,
	ds_write2_b32 = 14,
	ds_write2st64_b32 = 15,
	ds_cmpst_b32 = 16,
	ds_cmpst_f32 = 17,
	ds_min_f32 = 18,
	ds_max_f32 = 19,
	ds_nop = 20,
	ds_add_f32 = 21,
	ds_gws_sema_release_all = 24,
	ds_gws_init = 25,
	ds_gws_sema_v = 26,
	ds_gws_sema_br = 27,
	ds_gws_sema_p = 28,
	ds_gws_barrier = 29,
	ds_write_b8 = 30,
	ds_write_b16 = 31,
	ds_add_rtn_u32 = 32,
	ds_sub_rtn_u32 = 33,
	ds_rsub_rtn_u32 = 34,
	ds_inc_rtn_u32 = 35,
	ds_dec_rtn_u32 = 36,
	ds_min_rtn_i32 = 37,
	ds_max_rtn_i32 = 38,
	ds_min_rtn_u32 = 39,
	ds_max_rtn_u32 = 40,
	ds_and_rtn_b32 = 41,
	ds_or_rtn_b32 = 42,
	ds_xor_rtn_b32 = 43,
	ds_mskor_rtn_b32 = 44,
	ds_wrxchg_rtn_b32 = 45,
	ds_wrxchg2_rtn_b32 = 46,
	ds_wrxchg2st64_rtn_b32 = 47,
	ds_cmpst_rtn_b32 = 48,
	ds_cmpst_rtn_f32 = 49,
	ds_min_rtn_f32 = 50,
	ds_max_rtn_f32 = 51,
	ds_wrap_rtn_b32 = 52,
	ds_swizzle_b32 = 53,
	ds_read_b32 = 54,
	ds_read2_b32 = 55,
	ds_read2st64_b32 = 56,
	ds_read_i8 = 57,
	ds_read_u8 = 58,
	ds_read_i16 = 59,
	ds_read_u16 = 60,
	ds_consume = 61,
	ds_append = 62,
	ds_ordered_count = 63,
	ds_add_u64 = 64,
	ds_sub_u64 = 65,
	ds_rsub_u64 = 66,
	ds_inc_u64 = 67,
	ds_dec_u64 = 68,
	ds_min_i64 = 69,
	ds_max_i64 = 70,
	ds_min_u64 = 71,
	ds_max_u64 = 72,
	ds_and_b64 = 73,
	ds_or_b64 = 74,
	ds_xor_b64 = 75,
	ds_mskor_b64 = 76,
	ds_write_b64 = 77,
	ds_write2_b64 = 78,
	ds_write2st64_b64 = 79,
	ds_cmpst_b64 = 80,
	ds_cmpst_f64 = 81,
	ds_min_f64 = 82,
	ds_max_f64 = 83,
	ds_add_rtn_f32 = 85,
	ds_add_rtn_u64 = 96,
	ds_sub_rtn_u64 = 97,
	ds_rsub_rtn_u64 = 98,
	ds_inc_rtn_u64 = 99,
	ds_dec_rtn_u64 = 100,
	ds_min_rtn_i64 = 101,
	ds_max_rtn_i64 = 102,
	ds_min_rtn_u64 = 103,
	ds_max_rtn_u64 = 104,
	ds_and_rtn_b64 = 105,
	ds_or_rtn_b64 = 106,
	ds_xor_rtn_b64 = 107,
	ds_mskor_rtn_b64 = 108,
	ds_wrxchg_rtn_b64 = 109,
	ds_wrxchg2_rtn_b64 = 110,
	ds_wrxchg2st64_rtn_b64 = 111,
	ds_cmpst_rtn_b64 = 112,
	ds_cmpst_rtn_f64 = 113,
	ds_min_rtn_f64 = 114,
	ds_max_rtn_f64 = 115,
	ds_read_b64 = 118,
	ds_read2_b64 = 119,
	ds_read2st64_b64 = 120,
	ds_condxchg32_rtn_b64 = 126,
	ds_write_b8_d16_hi = 160,
	ds_write_b16_d16_hi = 161,
	ds_read_u8_d16 = 162,
	ds_read_u8_d16_hi = 163,
	ds_read_i8_d16 = 164,
	ds_read_i8_d16_hi = 165,
	ds_read_u16_d16 = 166,
	ds_read_u16_d16_hi = 167,
	ds_write_addtid_b32 = 176,
	ds_read_addtid_b32 = 177,
	ds_permute_b32 = 178,
	ds_bpermute_b32 = 179,
	ds_write_b96 = 222,
	ds_write_b128 = 223,
	ds_read_b96 = 254,
	ds_read_b128 = 255,
};

const char* ToString(ELDSGDSOps Op)
{
#define OP_TO_STRING_CASE(x) case ELDSGDSOps::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(ds_add_u32);
		OP_TO_STRING_CASE(ds_sub_u32);
		OP_TO_STRING_CASE(ds_rsub_u32);
		OP_TO_STRING_CASE(ds_inc_u32);
		OP_TO_STRING_CASE(ds_dec_u32);
		OP_TO_STRING_CASE(ds_min_i32);
		OP_TO_STRING_CASE(ds_max_i32);
		OP_TO_STRING_CASE(ds_min_u32);
		OP_TO_STRING_CASE(ds_max_u32);
		OP_TO_STRING_CASE(ds_and_b32);
		OP_TO_STRING_CASE(ds_or_b32);
		OP_TO_STRING_CASE(ds_xor_b32);
		OP_TO_STRING_CASE(ds_mskor_b32);
		OP_TO_STRING_CASE(ds_write_b32);
		OP_TO_STRING_CASE(ds_write2_b32);
		OP_TO_STRING_CASE(ds_write2st64_b32);
		OP_TO_STRING_CASE(ds_cmpst_b32);
		OP_TO_STRING_CASE(ds_cmpst_f32);
		OP_TO_STRING_CASE(ds_min_f32);
		OP_TO_STRING_CASE(ds_max_f32);
		OP_TO_STRING_CASE(ds_nop);
		OP_TO_STRING_CASE(ds_add_f32);
		OP_TO_STRING_CASE(ds_gws_sema_release_all);
		OP_TO_STRING_CASE(ds_gws_init);
		OP_TO_STRING_CASE(ds_gws_sema_v);
		OP_TO_STRING_CASE(ds_gws_sema_br);
		OP_TO_STRING_CASE(ds_gws_sema_p);
		OP_TO_STRING_CASE(ds_gws_barrier);
		OP_TO_STRING_CASE(ds_write_b8);
		OP_TO_STRING_CASE(ds_write_b16);
		OP_TO_STRING_CASE(ds_add_rtn_u32);
		OP_TO_STRING_CASE(ds_sub_rtn_u32);
		OP_TO_STRING_CASE(ds_rsub_rtn_u32);
		OP_TO_STRING_CASE(ds_inc_rtn_u32);
		OP_TO_STRING_CASE(ds_dec_rtn_u32);
		OP_TO_STRING_CASE(ds_min_rtn_i32);
		OP_TO_STRING_CASE(ds_max_rtn_i32);
		OP_TO_STRING_CASE(ds_min_rtn_u32);
		OP_TO_STRING_CASE(ds_max_rtn_u32);
		OP_TO_STRING_CASE(ds_and_rtn_b32);
		OP_TO_STRING_CASE(ds_or_rtn_b32);
		OP_TO_STRING_CASE(ds_xor_rtn_b32);
		OP_TO_STRING_CASE(ds_mskor_rtn_b32);
		OP_TO_STRING_CASE(ds_wrxchg_rtn_b32);
		OP_TO_STRING_CASE(ds_wrxchg2_rtn_b32);
		OP_TO_STRING_CASE(ds_wrxchg2st64_rtn_b32);
		OP_TO_STRING_CASE(ds_cmpst_rtn_b32);
		OP_TO_STRING_CASE(ds_cmpst_rtn_f32);
		OP_TO_STRING_CASE(ds_min_rtn_f32);
		OP_TO_STRING_CASE(ds_max_rtn_f32);
		OP_TO_STRING_CASE(ds_wrap_rtn_b32);
		OP_TO_STRING_CASE(ds_swizzle_b32);
		OP_TO_STRING_CASE(ds_read_b32);
		OP_TO_STRING_CASE(ds_read2_b32);
		OP_TO_STRING_CASE(ds_read2st64_b32);
		OP_TO_STRING_CASE(ds_read_i8);
		OP_TO_STRING_CASE(ds_read_u8);
		OP_TO_STRING_CASE(ds_read_i16);
		OP_TO_STRING_CASE(ds_read_u16);
		OP_TO_STRING_CASE(ds_consume);
		OP_TO_STRING_CASE(ds_append);
		OP_TO_STRING_CASE(ds_ordered_count);
		OP_TO_STRING_CASE(ds_add_u64);
		OP_TO_STRING_CASE(ds_sub_u64);
		OP_TO_STRING_CASE(ds_rsub_u64);
		OP_TO_STRING_CASE(ds_inc_u64);
		OP_TO_STRING_CASE(ds_dec_u64);
		OP_TO_STRING_CASE(ds_min_i64);
		OP_TO_STRING_CASE(ds_max_i64);
		OP_TO_STRING_CASE(ds_min_u64);
		OP_TO_STRING_CASE(ds_max_u64);
		OP_TO_STRING_CASE(ds_and_b64);
		OP_TO_STRING_CASE(ds_or_b64);
		OP_TO_STRING_CASE(ds_xor_b64);
		OP_TO_STRING_CASE(ds_mskor_b64);
		OP_TO_STRING_CASE(ds_write_b64);
		OP_TO_STRING_CASE(ds_write2_b64);
		OP_TO_STRING_CASE(ds_write2st64_b64);
		OP_TO_STRING_CASE(ds_cmpst_b64);
		OP_TO_STRING_CASE(ds_cmpst_f64);
		OP_TO_STRING_CASE(ds_min_f64);
		OP_TO_STRING_CASE(ds_max_f64);
		OP_TO_STRING_CASE(ds_add_rtn_f32);
		OP_TO_STRING_CASE(ds_add_rtn_u64);
		OP_TO_STRING_CASE(ds_sub_rtn_u64);
		OP_TO_STRING_CASE(ds_rsub_rtn_u64);
		OP_TO_STRING_CASE(ds_inc_rtn_u64);
		OP_TO_STRING_CASE(ds_dec_rtn_u64);
		OP_TO_STRING_CASE(ds_min_rtn_i64);
		OP_TO_STRING_CASE(ds_max_rtn_i64);
		OP_TO_STRING_CASE(ds_min_rtn_u64);
		OP_TO_STRING_CASE(ds_max_rtn_u64);
		OP_TO_STRING_CASE(ds_and_rtn_b64);
		OP_TO_STRING_CASE(ds_or_rtn_b64);
		OP_TO_STRING_CASE(ds_xor_rtn_b64);
		OP_TO_STRING_CASE(ds_mskor_rtn_b64);
		OP_TO_STRING_CASE(ds_wrxchg_rtn_b64);
		OP_TO_STRING_CASE(ds_wrxchg2_rtn_b64);
		OP_TO_STRING_CASE(ds_wrxchg2st64_rtn_b64);
		OP_TO_STRING_CASE(ds_cmpst_rtn_b64);
		OP_TO_STRING_CASE(ds_cmpst_rtn_f64);
		OP_TO_STRING_CASE(ds_min_rtn_f64);
		OP_TO_STRING_CASE(ds_max_rtn_f64);
		OP_TO_STRING_CASE(ds_read_b64);
		OP_TO_STRING_CASE(ds_read2_b64);
		OP_TO_STRING_CASE(ds_read2st64_b64);
		OP_TO_STRING_CASE(ds_condxchg32_rtn_b64);
		OP_TO_STRING_CASE(ds_write_b8_d16_hi);
		OP_TO_STRING_CASE(ds_write_b16_d16_hi);
		OP_TO_STRING_CASE(ds_read_u8_d16);
		OP_TO_STRING_CASE(ds_read_u8_d16_hi);
		OP_TO_STRING_CASE(ds_read_i8_d16);
		OP_TO_STRING_CASE(ds_read_i8_d16_hi);
		OP_TO_STRING_CASE(ds_read_u16_d16);
		OP_TO_STRING_CASE(ds_read_u16_d16_hi);
		OP_TO_STRING_CASE(ds_write_addtid_b32);
		OP_TO_STRING_CASE(ds_read_addtid_b32);
		OP_TO_STRING_CASE(ds_permute_b32);
		OP_TO_STRING_CASE(ds_bpermute_b32);
		OP_TO_STRING_CASE(ds_write_b96);
		OP_TO_STRING_CASE(ds_write_b128);
		OP_TO_STRING_CASE(ds_read_b96);
		OP_TO_STRING_CASE(ds_read_b128);
	default:
		return "UNKNOWN - LDS / GDS";
	}

#undef OP_TO_STRING_CASE
}

enum class EGLOBALOps : uint16
{
	global_load_ubyte = 8,
	global_load_sbyte = 9,
	global_load_ushort = 10,
	global_load_sshort = 11,
	global_load_dword = 12,
	global_load_dwordx2 = 13,
	global_load_dwordx4 = 14,
	global_load_dwordx3 = 15,
	global_load_dword_addtid = 22,
	global_store_dword_addtid = 23,
	global_store_byte = 24,
	global_store_byte_d16_hi = 25,
	global_store_short = 26,
	global_store_short_d16_hi = 27,
	global_store_dword = 28,
	global_store_dwordx2 = 29,
	global_store_dwordx4 = 30,
	global_store_dwordx3 = 31,
	global_load_ubyte_d16 = 32,
	global_load_ubyte_d16_hi = 33,
	global_load_sbyte_d16 = 34,
	global_load_sbyte_d16_hi = 35,
	global_load_short_d16 = 36,
	global_load_short_d16_hi = 37,
	global_atomic_swap = 48,
	global_atomic_cmpswap = 49,
	global_atomic_add = 50,
	global_atomic_sub = 51,
	global_atomic_csub = 52,
	global_atomic_smin = 53,
	global_atomic_umin = 54,
	global_atomic_smax = 55,
	global_atomic_umax = 56,
	global_atomic_and = 57,
	global_atomic_or = 58,
	global_atomic_xor = 59,
	global_atomic_inc = 60,
	global_atomic_dec = 61,
	global_atomic_fcmpswap = 62,
	global_atomic_fmin = 63,
	global_atomic_fmax = 64,
	global_atomic_swap_x2 = 80,
	global_atomic_cmpswap_x2 = 81,
	global_atomic_add_x2 = 82,
	global_atomic_sub_x2 = 83,
	global_atomic_smin_x2 = 85,
	global_atomic_umin_x2 = 86,
	global_atomic_smax_x2 = 87,
	global_atomic_umax_x2 = 88,
	global_atomic_and_x2 = 89,
	global_atomic_or_x2 = 90,
	global_atomic_xor_x2 = 91,
	global_atomic_inc_x2 = 92,
	global_atomic_dec_x2 = 93,
	global_atomic_fcmpswap_x2 = 94,
	global_atomic_fmin_x2 = 95,
	global_atomic_fmax_x2 = 96,
};

const char* ToString(EGLOBALOps Op)
{
#define OP_TO_STRING_CASE(x) case EGLOBALOps::x: return #x

	switch (Op)
	{
		OP_TO_STRING_CASE(global_load_ubyte);
		OP_TO_STRING_CASE(global_load_sbyte);
		OP_TO_STRING_CASE(global_load_ushort);
		OP_TO_STRING_CASE(global_load_sshort);
		OP_TO_STRING_CASE(global_load_dword);
		OP_TO_STRING_CASE(global_load_dwordx2);
		OP_TO_STRING_CASE(global_load_dwordx4);
		OP_TO_STRING_CASE(global_load_dwordx3);
		OP_TO_STRING_CASE(global_load_dword_addtid);
		OP_TO_STRING_CASE(global_store_dword_addtid);
		OP_TO_STRING_CASE(global_store_byte);
		OP_TO_STRING_CASE(global_store_byte_d16_hi);
		OP_TO_STRING_CASE(global_store_short);
		OP_TO_STRING_CASE(global_store_short_d16_hi);
		OP_TO_STRING_CASE(global_store_dword);
		OP_TO_STRING_CASE(global_store_dwordx2);
		OP_TO_STRING_CASE(global_store_dwordx4);
		OP_TO_STRING_CASE(global_store_dwordx3);
		OP_TO_STRING_CASE(global_load_ubyte_d16);
		OP_TO_STRING_CASE(global_load_ubyte_d16_hi);
		OP_TO_STRING_CASE(global_load_sbyte_d16);
		OP_TO_STRING_CASE(global_load_sbyte_d16_hi);
		OP_TO_STRING_CASE(global_load_short_d16);
		OP_TO_STRING_CASE(global_load_short_d16_hi);
		OP_TO_STRING_CASE(global_atomic_swap);
		OP_TO_STRING_CASE(global_atomic_cmpswap);
		OP_TO_STRING_CASE(global_atomic_add);
		OP_TO_STRING_CASE(global_atomic_sub);
		OP_TO_STRING_CASE(global_atomic_csub);
		OP_TO_STRING_CASE(global_atomic_smin);
		OP_TO_STRING_CASE(global_atomic_umin);
		OP_TO_STRING_CASE(global_atomic_smax);
		OP_TO_STRING_CASE(global_atomic_umax);
		OP_TO_STRING_CASE(global_atomic_and);
		OP_TO_STRING_CASE(global_atomic_or);
		OP_TO_STRING_CASE(global_atomic_xor);
		OP_TO_STRING_CASE(global_atomic_inc);
		OP_TO_STRING_CASE(global_atomic_dec);
		OP_TO_STRING_CASE(global_atomic_fcmpswap);
		OP_TO_STRING_CASE(global_atomic_fmin);
		OP_TO_STRING_CASE(global_atomic_fmax);
		OP_TO_STRING_CASE(global_atomic_swap_x2);
		OP_TO_STRING_CASE(global_atomic_cmpswap_x2);
		OP_TO_STRING_CASE(global_atomic_add_x2);
		OP_TO_STRING_CASE(global_atomic_sub_x2);
		OP_TO_STRING_CASE(global_atomic_smin_x2);
		OP_TO_STRING_CASE(global_atomic_umin_x2);
		OP_TO_STRING_CASE(global_atomic_smax_x2);
		OP_TO_STRING_CASE(global_atomic_umax_x2);
		OP_TO_STRING_CASE(global_atomic_and_x2);
		OP_TO_STRING_CASE(global_atomic_or_x2);
		OP_TO_STRING_CASE(global_atomic_xor_x2);
		OP_TO_STRING_CASE(global_atomic_inc_x2);
		OP_TO_STRING_CASE(global_atomic_dec_x2);
		OP_TO_STRING_CASE(global_atomic_fcmpswap_x2);
		OP_TO_STRING_CASE(global_atomic_fmin_x2);
		OP_TO_STRING_CASE(global_atomic_fmax_x2);
	default:
		return "UNKNOWN - GLOBAL";
	}

#undef OP_TO_STRING_CASE
}

inline bool HasTrailingLiteral(RDNA2::EVOP2Ops Op)
{
	switch (Op)
	{
	case RDNA2::EVOP2Ops::v_madmk_f32:
	case RDNA2::EVOP2Ops::v_madak_f32:
	case RDNA2::EVOP2Ops::v_fmamk_f32:
	case RDNA2::EVOP2Ops::v_fmaak_f32:
	case RDNA2::EVOP2Ops::v_fmamk_f16:
	case RDNA2::EVOP2Ops::v_fmaak_f16:
		return true;

	default:
		return false;
	}
}

void PrintSMEM(const FInstSMEM& Inst)
{
	ESMEMOps Op = (ESMEMOps)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintSOP1(const FInstSOP1& Inst)
{
	ESOP1Ops Op = (ESOP1Ops)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintSOP2(const FInstSOP2& Inst)
{
	ESOP2Ops Op = (ESOP2Ops)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintSOPP(const FInstSOPP& Inst)
{
	ESOPPOps Op = (ESOPPOps)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintSOPC(const FInstSOPC& Inst)
{
	ESOPCOps Op = (ESOPCOps)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintSOPK(const FInstSOPK& Inst)
{
	ESOPKOps Op = (ESOPKOps)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintMIMG(const FInstMIMG& Inst)
{
	EMIMGOps Op = (EMIMGOps)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintVINTERP(const FInstVINTERP& Inst)
{
	EVINTERPOps Op = (EVINTERPOps)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintVOP1(const FInstVOP1& Inst)
{
	EVOP1Ops Op = (EVOP1Ops)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintVOP2(const FInstVOP2& Inst)
{
	EVOP2Ops Op = (EVOP2Ops)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintVOP3A(const FInstVOP3A& Inst)
{
	const uint32_t VOP1Offset = 0x180;
	const uint32_t VOP2Offset = 0x100;

	if (Inst.OP >= VOP1Offset && (Inst.OP - VOP1Offset) <= 104 /* v_swapref_b32 */)
	{
		// VOP1 encoded as VOP3A
		EVOP1Ops Op = (EVOP1Ops)(Inst.OP - VOP1Offset);
		UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
	}
	else if (Inst.OP >= VOP2Offset && (Inst.OP - VOP2Offset) <= 60 /* v_pk_fmac_f16 */)
	{
		// VOP2 encoded as VOP3A
		EVOP2Ops Op = (EVOP2Ops)(Inst.OP - VOP2Offset);
		UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
	}
	else if (Inst.OP <= 255 /* v_cmpx_tru_f16 */)
	{
		// VOPC encoded as VOP3A
		EVOPCOps Op = (EVOPCOps)Inst.OP;
		UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
	}
	else
	{
		EVOP3ABOps Op = (EVOP3ABOps)Inst.OP;
		UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
	}
}

void PrintVOP3B(const FInstVOP3B& Inst)
{
	EVOP3ABOps Op = (EVOP3ABOps)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintVOP3P(const FInstVOP3P& Inst)
{
	EVOP3POps Op = (EVOP3POps)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintVOPC(const FInstVOPC& Inst)
{
	EVOPCOps Op = (EVOPCOps)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintLDSGDS(const FInstLDSGDS& Inst)
{
	ELDSGDSOps Op = (ELDSGDSOps)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintMUBUF(const FInstMUBUF& Inst)
{
	EMUBUFOps Op = (EMUBUFOps)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintMTBUF(const FInstMTBUF& Inst)
{
	EMTBUFOps Op = (EMTBUFOps)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintEXPORT(const FInstEXPORT& Inst)
{
	UE_LOGF(LogTemp, Warning, "exp");
}

void PrintFLAT(const FInstFSG& Inst)
{
	EFLATOps Op = (EFLATOps)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintSCRATCH(const FInstFSG& Inst)
{
	ESCRATCHOps Op = (ESCRATCHOps)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintGLOBAL(const FInstFSG& Inst)
{
	EGLOBALOps Op = (EGLOBALOps)Inst.OP;
	UE_LOGF(LogTemp, Warning, "%s", ToString(Op));
}

void PrintFSG(const FInstFSG& Inst)
{
	if (Inst.SEG == 0)
	{
		PrintFLAT(Inst);
	}
	else if (Inst.SEG == 1)
	{
		PrintSCRATCH(Inst);
	}
	else if (Inst.SEG == 2)
	{
		PrintGLOBAL(Inst);
	}
}

} // RDNA2