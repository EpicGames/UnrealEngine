// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

static const uint32 SOP2Header		= 0b10; // Then 7bits for Op
static const uint32 SOPKHeader		= 0b1011; // Then 5bits for Op
static const uint32 SOP1Header		= 0b101111101;
static const uint32 SOPCHeader		= 0b101111110;
static const uint32 SOPPHeader		= 0b101111111;
static const uint32 SMEMHeader		= 0b111101;
static const uint32 VOP2Header		= 0b0; // Then 6bits for Op
static const uint32 VOP1Header		= 0b0111111;
static const uint32 VOP3Header		= 0b110101;
static const uint32 VOPCHeader		= 0b0111110;
static const uint32 VOP3PHeader		= 0b110011;
static const uint32 VINTERPHeader	= 0b110010;
static const uint32 LDSGDSHeader	= 0b110110;
static const uint32 MUBUFHeader		= 0b111000;
static const uint32 MTBUFHeader		= 0b111010;
static const uint32 MIMGHeader		= 0b111100;
static const uint32 EXPORTHeader	= 0b111110;
static const uint32 FSGHeader		= 0b110111;

static const uint32 Operand_DPP8	= 233;
static const uint32 Operand_DPP8FI	= 234;
static const uint32 Operand_DPP16	= 250;
static const uint32 Operand_SDWA	= 249;
static const uint32 Operand_Literal	= 255;

enum class EInstructionType : uint8
{
	SOP2,
	SOPK,
	SOP1,
	SOPC,
	SOPP,
	SMEM,
	VOP2,
	VOP1,
	VOP3,
	VOPC,
	VOP3P,
	VINTERP,
	LDSGDS,
	MUBUF,
	MTBUF,
	MIMG,
	EXPORT,
	FSG,

	UNKNOWN,
};

struct FInstSOP2
{
	uint32 SSRC0 : 8;
	uint32 SSRC1 : 8;
	uint32 SDST : 7;
	uint32 OP : 7;
	uint32 ID : 2;
};

struct FInstSOPK
{
	uint32 SIMM16 : 16;
	uint32 SDST : 7;
	uint32 OP : 5;
	uint32 ID : 4;
};

struct FInstSOP1
{
	uint32 SSRC0 : 8;
	uint32 OP : 8;
	uint32 SDST : 7;
	uint32 ID : 9;
};

struct FInstSOPC
{
	uint32 SSRC0 : 8;
	uint32 SSRC1 : 8;
	uint32 OP : 7;
	uint32 ID : 9;
};

struct FInstSOPP
{
	uint32 SIMM16 : 16;
	uint32 OP : 7;
	uint32 ID : 9;
};

struct FInstSMEM
{
	uint32 SBASE : 6;
	uint32 SDATA : 7;
	uint32 UNUSED1 : 1;
	uint32 DLC : 1;
	uint32 UNUSED2 : 1;
	uint32 GLC : 1;
	uint32 UNUSED3 : 1;
	uint32 OP : 8;
	uint32 ID : 6;
	int32  OFFSET : 21; // signed
	uint32 UNUSED4 : 4;
	uint32 SOFFSET : 7;
};

struct FInstVOP2
{
	uint32 SRC0 : 9;
	uint32 VSRC1 : 8;
	uint32 VDST : 8;
	uint32 OP : 6;
	uint32 ID : 1;
};

struct FInstVOP3A
{
	uint32 VDST : 8;
	uint32 ABS : 3;
	uint32 OP_SEL : 4;
	uint32 CLMP : 1;
	uint32 OP : 10;
	uint32 ID : 6;
	uint32 SRC0 : 9;
	uint32 SRC1 : 9;
	uint32 SRC2 : 9;
	uint32 OMOD : 2;
	uint32 NEG : 3;
};

struct FInstVOP3B
{
	uint32 VDST : 8;
	uint32 SDST : 7;
	uint32 CLMP : 1;
	uint32 OP : 10;
	uint32 ID : 6;
	uint32 SRC0 : 9;
	uint32 SRC1 : 9;
	uint32 SRC2 : 9;
	uint32 OMOD : 2;
	uint32 NEG : 3;
};

struct FInstVOP3
{
	union FEncoding
	{
		FInstVOP3A VOP3A;
		FInstVOP3B VOP3B;
	}
	Encoding;
};

struct FInstVOP1
{
	uint32 SRC0 : 9;
	uint32 OP : 8;
	uint32 VDST : 8;
	uint32 ID : 7;
};

struct FInstVOPC
{
	uint32 SRC0 : 9;
	uint32 VSRC : 8;
	uint32 OP : 8;
	uint32 ID : 7;
};

struct FInstVOP3P
{
	uint32 VDST : 8;
	uint32 NEG_HI : 3;
	uint32 OP_SEL : 3;
	uint32 OP_SEL_HI : 1;
	uint32 CLMP : 1;
	uint32 OP : 7;
	uint32 Unused : 3;
	uint32 ID : 6;
	uint32 SRC0 : 9;
	uint32 SRC1 : 9;
	uint32 SRC2 : 9;
	uint32 OP_SEL_HI_2 : 2;
	uint32 NEG : 3;
};

struct FInstVINTERP
{
	uint32 VSRC : 8;
	uint32 ATTR_CHAN : 2;
	uint32 ATTR : 6;
	uint32 OP : 2;
	uint32 VDST : 8;
	uint32 ID : 6;
};

struct FInstLDSGDS
{
	uint32 OFFSET0 : 8;
	uint32 OFFSET1 : 8;
	uint32 UNUSED : 1;
	uint32 GDS : 1;
	uint32 OP : 8;
	uint32 ID : 6;
	uint32 ADDR : 8;
	uint32 DATA0 : 8;
	uint32 DATA1 : 8;
	uint32 VDST : 8;
};

struct FInstMUBUF
{
	uint32 OFFSET : 12;
	uint32 OFFEN : 1;
	uint32 IDXEN : 1;
	uint32 GLC : 1;
	uint32 DLC : 1;
	uint32 LDS : 1;
	uint32 UNUSED0 : 1;
	uint32 OP : 7;
	uint32 OPM : 1;
	uint32 ID : 6;
	uint32 VADDR : 8;
	uint32 VDATA : 8;
	uint32 SRSRC : 5;
	uint32 UNUSED1 : 1;
	uint32 SLC : 1;
	uint32 TFE : 1;
	uint32 SOFFSET : 8;
};

struct FInstMTBUF
{
	uint32 OFFSET : 12;
	uint32 OFFEN : 1;
	uint32 IDXEN : 1;
	uint32 GLC : 1;
	uint32 DLC : 1;
	uint32 OP : 3;
	uint32 FORMAT : 7;
	uint32 ID : 6;
	uint32 VADDR : 8;
	uint32 VDATA : 8;
	uint32 SRSRC : 5;
	uint32 OPM : 1;
	uint32 SLC : 1;
	uint32 TFE : 1;
	uint32 SOFFSET : 8;
};

struct FInstMIMG
{
	uint32 OPM : 1;
	uint32 NSA : 2;
	uint32 DIM : 3;
	uint32 UNUSED0 : 1;
	uint32 DLC : 1;
	uint32 DMASK : 4;
	uint32 UNRM : 1;
	uint32 GLC : 1;
	uint32 UNUSED1 : 1;
	uint32 R128 : 1;
	uint32 TFE : 1;
	uint32 LWE : 1;
	uint32 OP : 7;
	uint32 SLC : 1;
	uint32 ID : 6;
	uint32 VADDR : 8;
	uint32 VDATA : 8;
	uint32 SRSRC : 5;
	uint32 SSAMP : 5;
	uint32 UNUSED2 : 4;
	uint32 A16 : 1;
	uint32 D16 : 1;
	uint32 ADDR1 : 8;
	uint32 ADDR2 : 8;
	uint32 ADDR3 : 8;
	uint32 ADDR4 : 8;
	uint32 ADDR5 : 8;
	uint32 ADDR6 : 8;
	uint32 ADDR7 : 8;
	uint32 ADDR8 : 8;
	uint32 ADDR9 : 8;
	uint32 ADDR10 : 8;
	uint32 ADDR11 : 8;
	uint32 ADDR12 : 8;
};

struct FInstEXPORT
{
	uint32 EN : 4;
	uint32 TARGET : 6;
	uint32 COMPR : 1;
	uint32 DONE : 1;
	uint32 VM : 1;
	uint32 UNUSED : 13;
	uint32 ID : 6;
	uint32 VSRC0 : 8;
	uint32 VSRC1 : 8;
	uint32 VSRC2 : 8;
	uint32 VSRC3 : 8;
};

struct FInstFSG
{
	uint32 OFFSET : 12;
	uint32 DLC : 1;
	uint32 LDS : 1;
	uint32 SEG : 2;
	uint32 GLC : 1;
	uint32 SLC : 1;
	uint32 OP : 7;
	uint32 UNUSED0 : 1;
	uint32 ID : 6;
	uint32 ADDR8 : 8;
	uint32 DATA : 8;
	uint32 SADDR : 7;
	uint32 UNUSED1 : 1;
	uint32 VDST : 8;
};

struct FDPP16
{
	uint32 SRC0 : 8;
	uint32 DPPCTRL : 9;
	uint32 UNUSED : 1;
	uint32 FI : 1;
	uint32 BC : 1;
	uint32 SRC0_NEG : 1;
	uint32 SRC0_ABS : 1;
	uint32 SRC1_NEG : 1;
	uint32 SRC1_ABS : 1;
	uint32 BANK_MASK : 4;
	uint32 ROW_MASK : 4;
};

inline uint32 GetAdvanceAmount(EInstructionType InstructionType)
{
	switch (InstructionType)
	{
	case EInstructionType::SOP2: return 1u;
	case EInstructionType::SOPK: return 1u;
	case EInstructionType::SOP1: return 1u;
	case EInstructionType::SOPC: return 1u;
	case EInstructionType::SOPP: return 1u;
	case EInstructionType::SMEM: return 2u;
	case EInstructionType::VOP2: return 1u;
	case EInstructionType::VOP1: return 1u;
	case EInstructionType::VOP3: return 2u;
	case EInstructionType::VOPC: return 1u;
	case EInstructionType::VOP3P: return 2u;
	case EInstructionType::VINTERP: return 1u;
	case EInstructionType::LDSGDS: return 2u;
	case EInstructionType::MUBUF: return 2u;
	case EInstructionType::MTBUF: return 2u;
	case EInstructionType::MIMG: return 2u;
	case EInstructionType::EXPORT: return 2u;
	case EInstructionType::FSG: return 2u;
	default:
		break;
	}

	return 0xffffffffu; // Force termination
}

inline bool IsSOP2(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstSOP2& SOP2 = *reinterpret_cast<const FInstSOP2*>(Code);
	const bool bMatch = (SOP2.ID == SOP2Header);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::SOP2);
		if (SOP2.SSRC0 == Operand_Literal || SOP2.SSRC1 == Operand_Literal)
		{
			// Skip 32bit literal next in the instruction stream
			AdvanceAmount++;
		}
	}
	return bMatch;
}

inline bool IsSOPK(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstSOPK& SOPK = *reinterpret_cast<const FInstSOPK*>(Code);
	const bool bMatch = (SOPK.ID == SOPKHeader);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::SOPK);

		// SOPK do not ever have a trailing literal as per the spec
		// "Instructions in this format may not use a 32-bit literal constant which occurs immediately after the instruction."
	}
	return bMatch;
}

inline bool IsSOP1(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstSOP1& SOP1 = *reinterpret_cast<const FInstSOP1*>(Code);
	const bool bMatch = (SOP1.ID == SOP1Header);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::SOP1);
		if (SOP1.SSRC0 == Operand_Literal)
		{
			// Skip 32bit literal next in the instruction stream
			AdvanceAmount++;
		}
	}
	return bMatch;
}

inline bool IsSOPC(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstSOPC& SOPC = *reinterpret_cast<const FInstSOPC*>(Code);
	const bool bMatch = (SOPC.ID == SOPCHeader);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::SOPC);
		if (SOPC.SSRC0 == Operand_Literal || SOPC.SSRC1 == Operand_Literal)
		{
			// Skip 32bit literal next in the instruction stream
			AdvanceAmount++;
		}
	}
	return bMatch;
}

inline bool IsSOPP(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstSOPP& SOPP = *reinterpret_cast<const FInstSOPP*>(Code);
	const bool bMatch = (SOPP.ID == SOPPHeader);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::SOPP);
	}
	return bMatch;
}

inline bool IsSMEM(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstSMEM& SMEM = *reinterpret_cast<const FInstSMEM*>(Code);
	const bool bMatch = (SMEM.ID == SMEMHeader);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::SMEM);
	}
	return bMatch;
}

inline bool IsVOP2(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstVOP2& VOP2 = *reinterpret_cast<const FInstVOP2*>(Code);
	const bool bMatch = (VOP2.ID == VOP2Header);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::VOP2);
		if (VOP2.SRC0 == Operand_DPP8 || VOP2.SRC0 == Operand_DPP8FI || VOP2.SRC0 == Operand_DPP16 || VOP2.SRC0 == Operand_SDWA || VOP2.SRC0 == Operand_Literal)
		{
			AdvanceAmount++;
		}
	}
	return bMatch;
}

inline bool IsVOP3(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstVOP3A& VOP3 = *reinterpret_cast<const FInstVOP3A*>(Code);
	// VOP3A and VOP3B share the same header and bit range for the op code
	const bool bMatch = (VOP3.ID == VOP3Header);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::VOP3);
		if (VOP3.SRC0 == Operand_DPP8 || VOP3.SRC0 == Operand_DPP8FI || VOP3.SRC0 == Operand_DPP16 || VOP3.SRC0 == Operand_SDWA || VOP3.SRC0 == Operand_Literal ||
			VOP3.SRC1 == Operand_DPP8 || VOP3.SRC1 == Operand_DPP8FI || VOP3.SRC1 == Operand_DPP16 || VOP3.SRC1 == Operand_SDWA || VOP3.SRC1 == Operand_Literal ||
			VOP3.SRC2 == Operand_DPP8 || VOP3.SRC2 == Operand_DPP8FI || VOP3.SRC2 == Operand_DPP16 || VOP3.SRC2 == Operand_SDWA || VOP3.SRC2 == Operand_Literal)
		{
			AdvanceAmount++;
		}
	}
	return bMatch;
}

inline bool IsVOP1(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstVOP1& VOP1 = *reinterpret_cast<const FInstVOP1*>(Code);
	const bool bMatch = (VOP1.ID == VOP1Header);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::VOP1);
		if (VOP1.SRC0 == Operand_DPP8 || VOP1.SRC0 == Operand_DPP8FI || VOP1.SRC0 == Operand_DPP16 || VOP1.SRC0 == Operand_SDWA || VOP1.SRC0 == Operand_Literal)
		{
			AdvanceAmount++;
		}
	}
	return bMatch;
}

inline bool IsVOPC(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstVOPC& VOPC = *reinterpret_cast<const FInstVOPC*>(Code);
	const bool bMatch = (VOPC.ID == VOPCHeader);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::VOPC);
		if (VOPC.SRC0 == Operand_Literal || VOPC.SRC0 == Operand_SDWA)
		{
			AdvanceAmount++;
		}
	}
	return bMatch;
}

inline bool IsVOP3P(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstVOP3P& VOP3P = *reinterpret_cast<const FInstVOP3P*>(Code);
	const bool bMatch = (VOP3P.ID == VOP3PHeader);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::VOP3P);
	}
	return bMatch;
}

inline bool IsVINTERP(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstVINTERP& VINTERP = *reinterpret_cast<const FInstVINTERP*>(Code);
	const bool bMatch = (VINTERP.ID == VINTERPHeader);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::VINTERP);
	}
	return bMatch;
}

inline bool IsLDSGDS(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstLDSGDS& LDSGDS = *reinterpret_cast<const FInstLDSGDS*>(Code);
	const bool bMatch = (LDSGDS.ID == LDSGDSHeader);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::LDSGDS);
	}
	return bMatch;
}

inline bool IsMUBUF(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstMUBUF& MUBUF = *reinterpret_cast<const FInstMUBUF*>(Code);
	const bool bMatch = (MUBUF.ID == MUBUFHeader);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::MUBUF);
	}
	return bMatch;
}

inline bool IsMTBUF(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstMTBUF& MTBUF = *reinterpret_cast<const FInstMTBUF*>(Code);
	const bool bMatch = (MTBUF.ID == MTBUFHeader);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::MTBUF);
	}
	return bMatch;
}

inline bool IsMIMG(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstMIMG& MIMG = *reinterpret_cast<const FInstMIMG*>(Code);
	const bool bMatch = (MIMG.ID == MIMGHeader);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::MIMG);
		AdvanceAmount += MIMG.NSA; // Non sequential address (0-3 additional instruction dwords)
	}
	return bMatch;
}

inline bool IsEXPORT(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstEXPORT& EXPORT = *reinterpret_cast<const FInstEXPORT*>(Code);
	const bool bMatch = (EXPORT.ID == EXPORTHeader);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::EXPORT);
	}
	return bMatch;
}

inline bool IsFSG(const uint32* Code, uint32& AdvanceAmount)
{
	const FInstFSG& FSG = *reinterpret_cast<const FInstFSG*>(Code);
	const bool bMatch = (FSG.ID == FSGHeader);
	if (bMatch)
	{
		AdvanceAmount = GetAdvanceAmount(EInstructionType::FSG);
	}
	return bMatch;
}

inline EInstructionType DecodeInstructionType(const uint32* Code, uint32& AdvanceAmount)
{
	if (IsSOP1(Code, AdvanceAmount)) return EInstructionType::SOP1;
	if (IsSOPC(Code, AdvanceAmount)) return EInstructionType::SOPC;
	if (IsSOPP(Code, AdvanceAmount)) return EInstructionType::SOPP;
	if (IsSMEM(Code, AdvanceAmount)) return EInstructionType::SMEM;
	if (IsVOP1(Code, AdvanceAmount)) return EInstructionType::VOP1;
	if (IsVOP3(Code, AdvanceAmount)) return EInstructionType::VOP3;
	if (IsVOPC(Code, AdvanceAmount)) return EInstructionType::VOPC;
	if (IsVINTERP(Code, AdvanceAmount)) return EInstructionType::VINTERP;
	if (IsLDSGDS(Code, AdvanceAmount)) return EInstructionType::LDSGDS;
	if (IsMUBUF(Code, AdvanceAmount)) return EInstructionType::MUBUF;
	if (IsMTBUF(Code, AdvanceAmount)) return EInstructionType::MTBUF;
	if (IsMIMG(Code, AdvanceAmount)) return EInstructionType::MIMG;
	if (IsEXPORT(Code, AdvanceAmount)) return EInstructionType::EXPORT;
	if (IsFSG(Code, AdvanceAmount)) return EInstructionType::FSG;
	if (IsVOP3P(Code, AdvanceAmount)) return EInstructionType::VOP3P;
	if (IsSOPK(Code, AdvanceAmount)) return EInstructionType::SOPK;
	if (IsSOP2(Code, AdvanceAmount)) return EInstructionType::SOP2;
	if (IsVOP2(Code, AdvanceAmount)) return EInstructionType::VOP2;
	return EInstructionType::UNKNOWN;
}

inline const char* ToString(EInstructionType Inst)
{
#define OP_TO_STRING_CASE(x) case EInstructionType::x: return #x

	switch (Inst)
	{
		OP_TO_STRING_CASE(SOP2);
		OP_TO_STRING_CASE(SOPK);
		OP_TO_STRING_CASE(SOP1);
		OP_TO_STRING_CASE(SOPC);
		OP_TO_STRING_CASE(SOPP);
		OP_TO_STRING_CASE(SMEM);
		OP_TO_STRING_CASE(VOP2);
		OP_TO_STRING_CASE(VOP1);
		OP_TO_STRING_CASE(VOP3);
		OP_TO_STRING_CASE(VOPC);
		OP_TO_STRING_CASE(VOP3P);
		OP_TO_STRING_CASE(VINTERP);
		OP_TO_STRING_CASE(LDSGDS);
		OP_TO_STRING_CASE(MUBUF);
		OP_TO_STRING_CASE(MTBUF);
		OP_TO_STRING_CASE(MIMG);
		OP_TO_STRING_CASE(EXPORT);
		OP_TO_STRING_CASE(FSG);

	default:
		return "UNKNOWN";
	}

#undef OP_TO_STRING_CASE
}