// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/ContainerAllocationPolicies.h"
#include "Misc/MemStack.h"
#include "Misc/CString.h"

/** Enable logging for debugging */
#define UE_SHADER_SDCE_LOG_ALL        0
#define UE_SHADER_SDCE_LOG_STRUCTURAL 0
#define UE_SHADER_SDCE_LOG_COMPLEX    0

/** If true, only populate symbols for tagged types, speeds up parsing but may miss nested type handling */
#define UE_SHADER_SDCE_TAGGED_ONLY 0

namespace UE::ShaderMinifier::SDCE
{
	/** Simple mutable view */
	template<typename T>
	struct TMutableStringView
	{
		TMutableStringView() : Begin(nullptr), Length(0)
		{
			
		}

		TMutableStringView(TString<T>& InView) : Begin(InView.GetCharArray().GetData()), Length(InView.Len())
		{
			
		}

		TMutableStringView(T* InBegin, int32 InLength) : Begin(InBegin), Length(InLength)
		{
			
		}

		TMutableStringView(T* InBegin) : TMutableStringView(InBegin, TCString<T>::Strlen(InBegin))
		{
			
		}

		T* begin() const { return Begin; }
		T* end()   const { return Begin + Length; }
		
		T* Begin;
		int32 Length;
	};

	/**
	 * Note that while the minifier parses contents on its ansi sources,
	 * SDCE is invoked during actual shader compilation which acts on wide
	 * characters. Despite that, we implicitly assume that tokenization
	 * segment lookups may operate on their ansi counterpart without
	 * accounting for encoding differences (i.e., the wide character is ansi)
	 */
	using FParseCharType      = TCHAR;
	using FParseViewType      = TMutableStringView<FParseCharType>;
	using FParseConstViewType = TStringView<FParseCharType>;
	using FParseStringType    = TString<FParseCharType>;
	using FLookupCharType     = ANSICHAR;
	using FLookupViewType     = TStringView<FLookupCharType>;

	/**
	 * Shader metadata prefix, e.g., UESHADERMETADATA_SDCE <name>
	 */
	static const TCHAR* ShaderMetadataPrefix = TEXT("SDCE ");
	
	using FMemStackSetAllocator = TSetAllocator<TSparseArrayAllocator<TMemStackAllocator<>, TMemStackAllocator<>>, TMemStackAllocator<>>;
}
