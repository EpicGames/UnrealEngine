// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ShaderMinifier.h"
#include "SDCEToken.h"
#include "SDCEShallowParser.h"
#include "SDCESymbolContext.h"
#include "SDCESemanticContext.h"

namespace UE::ShaderMinifier::SDCE
{
	/**
	 * Symbolic Dead-Code-Elimination Context
	 * Effectively a shallow parser with in-place semantics.
	 */
	class FContext
	{
		// In place parsing
		friend class FShallowParser;
		
	public:
		FContext(const TConstArrayView<FParseConstViewType>& DCESymbols) : SemanticCtx(SymbolCtx)
		{
			// Setup tagged type set
			for (FParseConstViewType SymbolicDceStructName : DCESymbols)
			{
				TaggedTypes.Add(FConstSmallStringView::Text(SymbolicDceStructName.begin(), SymbolicDceStructName.Len()));
			}
		}

		/**
		 * Parse a given code chunk and generate relevant semantics
		 */
		void Analyze(const FParseViewType& Code)
		{
			// In-place parsing
			Parser.ParseCode(Code, *this);
		}

		/**
		 * Minify all parsed chunks in-place against their stores
		 */
		void MinifyInPlace()
		{
			// Eliminate in-order, lets us get away with a single text cursor
			SymbolCtx.StoreSites.Sort([](const FMemberStoreSite& A, const FMemberStoreSite& B)
			{
				return A.AssignmentSite.Begin < B.AssignmentSite.Begin;
			});

			// Iteratively eliminate the realized loads of assignments we know are to be DCE'd
			// i.e., If we have:
			//   Data.A = 41;
			//   Data.B = Data.A + 1;
			//
			// We know B is to be DCE'd, so we remove the realized loads on its RHS value span.
			// After which A is known unrealized. And so on.
			for (;;)
			{
				bool bMutatedRealizedLoad = false;

				// Try to check for new unrealized loads
				for (FMemberStoreSite& StoreSite : SymbolCtx.StoreSites)
				{
					check(StoreSite.AddressSymbol->bTagged);

					// Still in use?
					if (StoreSite.bVisited || StoreSite.AddressSymbol->Variable.RealizedLoadCount > 0)
					{
						continue;
					}

					// Mark as visited, fine before complex check since it's immutable
					StoreSite.bVisited = true;

					// Check if the symbol is considered "complex", i.e., the identifier was used in a
					// code section that we couldn't parse or derive semantics from
					StoreSite.bIsResolvedToComplex = SymbolCtx.ComplexBaseSet.Contains(StoreSite.AddressSymbol->ID);
					if (StoreSite.bIsResolvedToComplex)
					{
						continue;
					}

					// Remove all the loads on the RHS
					SemanticCtx.RemoveRealizedLoads(StoreSite.ValueSpan);
					bMutatedRealizedLoad = true;
				}

				if (!bMutatedRealizedLoad)
				{
					break;
				}
			}

			// Finally, eliminate!
			for (const FMemberStoreSite& StoreSite : SymbolCtx.StoreSites)
			{
				check(StoreSite.AddressSymbol->bTagged);

				// Ignore if the address has a realized load or complex in nature
				if (StoreSite.AddressSymbol->Variable.RealizedLoadCount > 0 || StoreSite.bIsResolvedToComplex)
				{
					continue;
				}
				
				// Replace with spaces while maintaining line numbers
				for (int32 i = 0; i < StoreSite.AssignmentSite.Length; i++)
				{
					if (StoreSite.AssignmentSite.Begin[i] != '\n')
					{
						StoreSite.AssignmentSite.Begin[i] = ' ';
					}
				}
			}
		}
	
	private: /** Struct Visitors */
		
		void VisitStructPrologue(const FSmallStringView& ID)
		{
			// Push the stack
			FSymbolScope& Scope = SymbolCtx.SymbolStack.Emplace_GetRef();

			// Is this a tagged type?
			const bool bTagged = TaggedTypes.Contains(ID);
			
	#if UE_SHADER_SDCE_LOG_STRUCTURAL
			UE_LOGF(LogTemp, Error, ">> ENTER %s", *FParseStringType::ConstructFromPtrSize(ID.Begin, ID.Length));
	#endif // UE_SHADER_SDCE_LOG_STRUCTURAL

#if UE_SHADER_SDCE_TAGGED_ONLY
			if (!bTagged)
			{
				return;
			}
#endif // UE_SHADER_SDCE_TAGGED_ONLY
			
			// Assign composite of current stack
			FSymbol* Symbol = SymbolCtx.CreateSymbol(ESymbolKind::Composite);
			Symbol->ID = ID;
			Symbol->bTagged = bTagged;
			Scope.Composite = Symbol;
	
			// Add to parent stack
			SymbolCtx.SymbolStack[SymbolCtx.SymbolStack.Num() - 2].Symbols.Add(ID, Symbol);
		}
		
		void VisitStructEpilogue()
		{
			SymbolCtx.SymbolStack.Pop();
		}
	
	private: /** Function Visitors */
	
		void VisitFunctionPrologue()
		{
			// Push the stack
			SymbolCtx.SymbolStack.Emplace();
	
	#if UE_SHADER_SDCE_LOG_STRUCTURAL
			FSmallStringView ID = SemanticCtx.GetOp(SemanticCtx.OpCount() - 1).ID;
			UE_LOGF(LogTemp, Error, ">> ENTER %s", *FParseStringType::ConstructFromPtrSize(ID.Begin, ID.Length));
	#endif // UE_SHADER_SDCE_LOG_STRUCTURAL
	
			// Not semantically relevant for now
			SemanticCtx.Empty();
		}
	
		void VisitFunctionEpilogue()
		{
			SymbolCtx.SymbolStack.Pop();
		}
	
	private: /** Expression Visitors */
		
		void VisitDeclaration()
		{
			FSemanticOp& Node = SemanticCtx.PushOp();
			Node.Type = ESemanticOp::Declare;
			Node.Precedence = EPrecedence::Declare;
		}
	
		void VisitDrain(EPrecedence Precedence, const TCHAR* DebugReason)
		{
			FSemanticOp& DrainNode = SemanticCtx.PushOp();
			DrainNode.Type = ESemanticOp::Drain;
			DrainNode.Precedence = Precedence;
			DrainNode.Drain.Count = 1;
			DrainNode.Drain.DebugReason = DebugReason;
		}
	
		void VisitType(const FSmallStringView& Type)
		{
			FSemanticOp& Node = SemanticCtx.PushOp();
			Node.Type = ESemanticOp::Type;
			Node.Precedence = EPrecedence::Primary;
			Node.ID = Type;
		}
		
		void VisitAlias()
		{
			FSymbolScope& Scope = SymbolCtx.SymbolStack.Last();
	
#if 0
			// Add alias
			if (FSymbol* Symbol = FindSymbol(Type.ID, ESymbolKind::Composite))
			{
				Scope.Symbols.Add(ID.ID, Symbol);
			}
#endif
		}
	
		void VisitFramePrologue()
		{
			FSemanticOp& Node = SemanticCtx.PushOp();
			Node.Type = ESemanticOp::FramePrologue;
		}
	
		void VisitFrameEpilogue()
		{
			FSemanticOp& Node = SemanticCtx.PushOp();
			Node.Type = ESemanticOp::FrameEpilogue;
		}
	
		void VisitAggregateComposite()
		{
			FSemanticOp& Node = SemanticCtx.PushOp();
			Node.Type = ESemanticOp::ID;
			Node.Precedence = EPrecedence::Primary;
			Node.ID = FSmallStringView{};
		}
	
		void VisitPrimaryMemberAccess(const FSmallStringView& ID)
		{
			FSemanticOp& NodeAccess = SemanticCtx.PushOp();
			NodeAccess.Type = ESemanticOp::Access;
			NodeAccess.Precedence = EPrecedence::Access;
			
			FSemanticOp& Node = SemanticCtx.PushOp();
			Node.Type = ESemanticOp::ID;
			Node.Precedence = EPrecedence::Primary;
			Node.ID = ID;
		}
	
		void VisitPrimaryID(const FSmallStringView& ID)
		{
			FSemanticOp& Node = SemanticCtx.PushOp();
			Node.Type = ESemanticOp::ID;
			Node.Precedence = EPrecedence::Primary;
			Node.ID = ID;
		}
	
		void VisitComplexID(const FSmallStringView& ID)
		{
			// The parser has deemed this token complex, treat all accessors into said name as complex
			SymbolCtx.AddComplexAccessor(ID);
		}
	
		void VisitBinary(FToken Op)
		{
			FSemanticOp& Node = SemanticCtx.PushOp();
			Node.Type = ESemanticOp::Binary;
			Node.Precedence = Op.Type == ETokenType::BinaryEq ? EPrecedence::BinaryAssign : EPrecedence::BinaryGeneric;
			Node.Binary.Op = Op;
		}
	
	private: /** Statement Visitors */
	
		void VisitStatementPrologue(FParseCharType* Cursor)
		{
			CurrentStatementPrologueCursor = Cursor;
			
			// Just flush the previous contents
			SemanticCtx.Empty();
		}
	
		void VisitStatementEpilogue(FParseCharType* Cursor)
		{
#if UE_BUILD_DEBUG
			// Useful for debugging
			static uint32 DebugCounter = 0;
			++DebugCounter;
#endif // NDEBUG

			// Evaluate the current op-stack
			SemanticCtx.Evaluate(
				FParseViewType(
					CurrentStatementPrologueCursor,
					Cursor - CurrentStatementPrologueCursor
				)
			);
	
	#if UE_SHADER_SDCE_LOG_ALL
			UE_LOGF(LogTemp, Error, "End of statement");
	#endif // UE_SHADER_SDCE_LOG_ALL
		}

	private:
		/** Statement prologue cursor */
		FParseCharType* CurrentStatementPrologueCursor = nullptr;

	private:
		/** Shared shallow parser */
		FShallowParser Parser;

		/** Contexts */
		FSymbolContext   SymbolCtx;
		FSemanticContext SemanticCtx;

		/** Actual set of types we're minifying */
		TSet<FConstSmallStringView, FConstSmallStringKeyFuncs, FMemStackSetAllocator> TaggedTypes;
	};
}
