// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/MemStack.h"

namespace UE::ShaderMinifier::SDCE
{
	template<typename T, uint32 ChunkSize>
	struct TOpBuffer
	{
		/** Emplace a new op */
		T& EmplaceUninitialized()
		{
			int32 ChunkIndex = Count / ChunkSize;
			int32 OpIndex    = Count % ChunkSize;
	
			if (!Chunks.IsValidIndex(ChunkIndex))
			{
				for (int32 i = Chunks.Num(); i <= ChunkIndex; i++)
				{
					Chunks.Add(static_cast<Chunk*>(FMemStack::Get().Alloc(sizeof(Chunk), alignof(Chunk))));
				}
			}
	
			T& Op = Chunks[ChunkIndex]->Ops[OpIndex];
			Count++;
			return Op;
		}

		/** Add a new chunk value */
		void Add(const T& Value)
		{
			new (&EmplaceUninitialized()) T(Value);
		}

		/** Get an op by its index */
		const T& operator[](int32 Index) const
		{
			int32 ChunkIndex = Index / ChunkSize;
			int32 OpIndex    = Index % ChunkSize;
			return Chunks[ChunkIndex]->Ops[OpIndex];
		}

		/** Get an op by its index */
		T& operator[](int32 Index)
		{
			int32 ChunkIndex = Index / ChunkSize;
			int32 OpIndex    = Index % ChunkSize;
			return Chunks[ChunkIndex]->Ops[OpIndex];
		}

		/** Any ops? */
		bool IsEmpty() const
		{
			return Count == 0;
		}

		/** Set the number of ops */
		void SetNumNoAlloc(int32 InCount)
		{
			check(InCount >= 0 && InCount <= static_cast<int32>(Chunks.Num() * ChunkSize));
			Count = InCount;
		}
	
		/** Clear this buffer */
		void Empty()
		{
			Count = 0;
		}
	
		/** Number of available ops */
		int32 Num() const
		{
			return Count;
		}

	private:
		struct Chunk
		{
			T Ops[ChunkSize];
		};
	
		TArray<Chunk*, TMemStackAllocator<>> Chunks;
		
		int32 Count = 0;
	};
}
