// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SDCECommon.h"
#include "SDCEShallowParser.h"
#include "SDCESymbolContext.h"
#include "SDCEOpBuffer.h"
#include "SDCEToken.h"

namespace UE::ShaderMinifier::SDCE
{
	enum class ESemanticOp
	{
		ID,
		Type,
		Access,
		Binary,
		Declare,
		Drain,
		FramePrologue,
		FrameEpilogue,
	};
	
	struct FSemanticOp
	{
		ESemanticOp Type;
		EPrecedence Precedence;

		/** Payload */
		union
		{
			/** ESemanticOp::ID */
			FSmallStringView ID;
				
			/** ESemanticOp::Binary */
			struct
			{
				FToken Op;
			} Binary;
	
			/** ESemanticOp::Drain */
			struct
			{
				int32 Count;
				const TCHAR* DebugReason;
			} Drain;
		};
	};
	
	static_assert(sizeof(FSemanticOp) == 24, "Unexpected FSemanticOp packing");

	/**
	 * Simple semantic evaluator, reduces a set of semantic operations by their natural precedence
	 * with a naive shunting yard approach. This was picked over an "AST" like structure to keep
	 * things as lightweight as possible.
	 */
	class FSemanticContext
	{
	public:
		FSemanticContext(FSymbolContext& SymCtx) : SymCtx(SymCtx)
		{
			
		}

		/** Clear the context */
		void Empty()
		{
			OpBuffer.Empty();
		}

		/** Create a new op */
		FSemanticOp& PushOp()
		{
			return OpBuffer.EmplaceUninitialized();
		}

		const FSemanticOp& GetOp(int32 Index) const
		{
			return OpBuffer[Index];
		}

		int32 OpCount() const
		{
			return OpBuffer.Num();
		}

		/** Evaluate all pushed ops */
		void Evaluate(const FParseViewType& StatementRegion)
		{
			CurrentStatementRegion = StatementRegion;

			// Early out
			if (OpBuffer.IsEmpty())
			{
				return;
			}

			const int32 ValueHistoryBegin = ValueHistory.Num();
	
			// Single pass tape evaluation
			for (int32 OpIndex = 0; OpIndex < OpBuffer.Num(); OpIndex++)
			{
				const FSemanticOp& Buffer = OpBuffer[OpIndex];
				switch (Buffer.Type)
				{
					case ESemanticOp::ID:
					{
						FStackValue Value;
						Value.ID = Buffer.ID;
						Value.Span = FValueSpan(ValueHistory.Num(), ValueHistory.Num() + 1);
						PushValue(Value);
						break;
					}
					case ESemanticOp::Type:
					{
						FStackValue Value;
						Value.ID = Buffer.ID;
						Value.Span = FValueSpan(ValueHistory.Num(), ValueHistory.Num() + 1);
						PushValue(Value);
						break;
					}
					case ESemanticOp::Access:
					{
						FReduceOp Op;
						Op.Type = ESemanticOp::Access;
						Op.Precedence = EPrecedence::Access;
						PushOp(Op);
						break;
					}
					case ESemanticOp::Binary:
					{
						FReduceOp Op;
						Op.Type = ESemanticOp::Binary;
						Op.Precedence = Buffer.Precedence;
						Op.Binary.Op = Buffer.Binary.Op;
						PushOp(Op);
						break;
					}
					case ESemanticOp::Declare:
					{
						FReduceOp Op;
						Op.Type = ESemanticOp::Declare;
						Op.Precedence = EPrecedence::Declare;
						PushOp(Op);
						break;
					}
					case ESemanticOp::Drain:
					{
						FReduceOp Op;
						Op.Type = ESemanticOp::Drain;
						Op.Precedence = Buffer.Precedence;
						Op.Drain.Count = Buffer.Drain.Count;
						PushOp(Op);
						break;
					}
					case ESemanticOp::FramePrologue:
					{
						FFrameHead Head;
						Head.OpHead = OpStack.Num();
						Head.ValueHead = ValueStack.Num();
						FrameStack.Add(Head);
						break;
					}
					case ESemanticOp::FrameEpilogue:
					{
						FFrameHead Head = FrameStack.Pop();

						// Number of operands we need to drain
						int32 PendingOps = OpStack.Num() - Head.OpHead;
						check(PendingOps >= 0);

						// Keep draining
						for (int32 i = 0; i < PendingOps; i++)
						{
							Reduce();
						}
	
						check(OpStack.Num() == Head.OpHead);
						break;
					}
				}
			}

			// Reduce all pending ops
			while (!OpStack.IsEmpty())
			{
				Reduce();
			}

			// Go through all accessed values, and treat all read sites as loaded
			for (int32 i = ValueHistoryBegin; i < ValueHistory.Num(); i++)
			{
				const FStackValue& Value = ValueHistory[i];
				if (!Value.bIsReadSite)
				{
					continue;
				}

				// Only tracking for variables currently
				if (Value.Symbol && Value.Symbol->Kind == ESymbolKind::Variable)
				{
					Value.Symbol->Variable.RealizedLoadCount++;
				}
			}
			
			// Mismatched drain, somewhere
			// Note that statements may reduce to a single value, that's completely fine
			check(ValueStack.Num() <= 1);

			// Cleanup
			OpBuffer.Empty();
			ValueStack.Empty();
			OpStack.Empty();
			FrameStack.Empty();
	
	#if UE_SHADER_SDCE_LOG_ALL
			UE_LOGF(LogTemp, Error, "End of statement");
	#endif // UE_SHADER_SDCE_LOG_ALL
		}

		/** Remove all the realized from a span of values */
		void RemoveRealizedLoads(const FValueSpan& Span)
		{
			for (int32 i = Span.Begin; i < Span.End; i++)
			{
				FStackValue& Value = ValueHistory[i];
				if (!Value.bIsReadSite)
				{
					continue;
				}

				// Only tracking for variables currently
				if (Value.Symbol && Value.Symbol->Kind == ESymbolKind::Variable)
				{
					Value.Symbol->Variable.RealizedLoadCount--;
				}
			}
		}

	private:
		/** Internal reduction op */
		struct FReduceOp
		{
			ESemanticOp Type;
			EPrecedence Precedence;

			/** Payload */
			union
			{
				struct
				{
					FToken Op;
				} Binary;
					
				struct
				{
					int32 Count;
				} Drain;
			};
		};
	
		static_assert(sizeof(FReduceOp) == 24, "Unexpected FReduceOp packing");

	private:
		/** Reduce the top most op */
		void Reduce()
		{
			FReduceOp Op = OpStack.Pop();
			switch (Op.Type)
			{
				default:
				{
					checkNoEntry();
				}
				case ESemanticOp::Access:
				{					
					// Stack stack
					FStackValue RHS = ValueStack.Pop();
					FStackValue LHS = ValueStack.Pop();

					// Resolve it
					FSymbol* LHSSymbol = ResolveValueSymbol(LHS);
					if (!LHSSymbol)
					{
#if UE_SHADER_SDCE_LOG_COMPLEX
						UE_LOGF(LogTemp, Error, "LHS Complex: Failed to resolve");
#endif // UE_SHADER_SDCE_LOG_COMPLEX

						// Failed to resolve, treat as complex
						PushUnresolvedValue(LHS.Span | RHS.Span);
						SymCtx.AddComplexAccessor(RHS.ID);
						break;
					}

					// Unwrap variables
					if (LHSSymbol->Kind == ESymbolKind::Variable)
					{
						// If nested, treat LHS as loaded
						if (LHSSymbol->bTagged)
						{
							// TODO: Is this correct? I can't remember why I added this...
							LHSSymbol->Variable.RealizedLoadCount++;
						}

						// Typeless? Treat as complex
						LHSSymbol = LHSSymbol->Variable.Type;
						if (!LHSSymbol)
						{
#if UE_SHADER_SDCE_LOG_COMPLEX
							UE_LOGF(LogTemp, Error, "LHS Complex: Failed to resolve variable type");
#endif // UE_SHADER_SDCE_LOG_COMPLEX
							
							PushUnresolvedValue(LHS.Span | RHS.Span);
							SymCtx.AddComplexAccessor(RHS.ID);
							break;
						}
					}

					// Find the contained symbol, may fail
					FSymbol* ContainedSymbol = SymCtx.LinearCompositeFindSymbol(LHSSymbol, RHS.ID);

					// Setup 
					FStackValue Value;
					Value.Symbol = ContainedSymbol;
					Value.Span = FValueSpan(ValueHistory.Num(), ValueHistory.Num() + 1);
					Value.Span = LHS.Span | Value.Span;
					Value.bIsReadSite = true;
					PushValue(Value);

#if UE_SHADER_SDCE_LOG_ALL
					if (LHSSymbol && ContainedSymbol)
					{
						UE_LOGF(LogTemp, Error, "Access %s . %s", *FParseStringType::ConstructFromPtrSize(LHSSymbol->ID.Begin, LHSSymbol->ID.Length), *FParseStringType::ConstructFromPtrSize(ContainedSymbol->ID.Begin, ContainedSymbol->ID.Length));
					}
#endif // UE_SHADER_SDCE_LOG_ALL

					break;
				}
				case ESemanticOp::Binary:
				{
					FStackValue RHS = ValueStack.Pop();
					FStackValue LHS = ValueStack.Pop();

					// Resolve the operands (RHS for load tracking)
					FSymbol* LHSSymbol = ResolveValueSymbol(LHS);
					FSymbol* RHSSymbol = ResolveValueSymbolReadSite(RHS);

					// If non-assignment, just ignore
					if (Op.Binary.Op.Type != ETokenType::BinaryEq)
					{
						PushUnresolvedValue(LHS.Span | RHS.Span);
						break;
					}

					// Did we resolve the LHS?
					if (LHSSymbol)
					{
						// If not tagged, ignore
						if (!LHSSymbol->bTagged)
						{
							PushUnresolvedValue(LHS.Span | RHS.Span);
							break;
						}

						// For now, only DCE simple store sites
						// Will be expanded on in the future
						if (IsTrivialNonCompoundAssignment())
						{
							FMemberStoreSite& Site = SymCtx.StoreSites.Emplace_GetRef();
							Site.AddressSymbol = LHSSymbol;
							Site.ValueSymbol = RHSSymbol;
							Site.AddressSpan = LHS.Span;
							Site.ValueSpan = RHS.Span;
							Site.AssignmentSite.Begin  = CurrentStatementRegion.Begin;
							Site.AssignmentSite.Length = static_cast<uint16_t>(CurrentStatementRegion.Length);
						}
					}

#if UE_SHADER_SDCE_LOG_ALL
					if (LHSSymbol && RHSSymbol)
					{
						UE_LOGF(LogTemp, Error, "BINARY %s = %s", *FParseStringType::ConstructFromPtrSize(LHSSymbol->ID.Begin, LHSSymbol->ID.Length), *FParseStringType::ConstructFromPtrSize(RHS.ID.Begin, RHS.ID.Length));
					}
#endif // UE_SHADER_SDCE_LOG_ALL

					// Mark all variables on the LHS as conditional
					// Need to check for side effects, obviously
					for (int32 i = LHS.Span.Begin; i < LHS.Span.End; i++)
					{
						FStackValue& Value = ValueHistory[i];
						Value.bIsReadSite = false;
					}

					// TODO: Add comment about side effects
					LHS.bIsReadSite = false;

					// Assignment propagates LHS
					PushValue(LHS);
					break;
				}
				case ESemanticOp::Declare:
				{
					FSymbolScope& Scope = SymCtx.SymbolStack.Last();

					// Stack stack
					FStackValue ID   = ValueStack.Pop();
					FStackValue Type = ValueStack.Pop();

					// Find the type
					FSymbol* TypeSymbol = SymCtx.FindSymbol(Type.ID);

					// Tagged status, for now we only tag declarations inside of composites
					// Function local declarations require qualifier knowledge, which is a TODO
					bool bTagged = false;
					if (Scope.Composite)
					{
						bTagged |= Scope.Composite->bTagged;
						
						if (TypeSymbol)
						{
							bTagged |= TypeSymbol->bTagged;
						}
					}

#if UE_SHADER_SDCE_TAGGED_ONLY
					if (!bTagged)
					{
						PushUnresolvedValue();
						break;
					}
#endif // UE_SHADER_SDCE_TAGGED_ONLY

					// Add to current composite
					FSymbol* Symbol = SymCtx.CreateSymbol(ESymbolKind::Variable);
					Symbol->ID = ID.ID;
					Symbol->Variable.Type = TypeSymbol;
					Symbol->bTagged = bTagged;

					// Current scope
					if (Scope.Composite)
					{
						Scope.Composite->Composite.ContainedSymbols.Add(Symbol);
					}

					// Add lookup to stack
					Scope.Symbols.Add(ID.ID, Symbol);

#if UE_SHADER_SDCE_LOG_ALL
					UE_LOGF(LogTemp, Error, "DECLARE %s %s", *FParseStringType::ConstructFromPtrSize(Type.ID.Begin, Type.ID.Length), *FParseStringType::ConstructFromPtrSize(ID.ID.Begin, ID.ID.Length));
#endif // UE_SHADER_SDCE_LOG_ALL

					FStackValue Value;
					Value.Symbol = Symbol;
					Value.Span = FValueSpan(ValueHistory.Num(), ValueHistory.Num() + 1);
					PushValue(Value);
					break;
				}
				case ESemanticOp::Drain:
				{
					FValueSpan Span;

					// On shallow parsing drains, just treat the symbol, if any, as loaded
					for (int32 i = 0; i < Op.Drain.Count; i++)
					{
						FStackValue Value = ValueStack.Pop();

						// Resolve the symbol and treat it as a read site
						ResolveValueSymbolReadSite(Value);

						// Keep track of the value span
						if (i == 0)
						{
							Span = Value.Span;
						}
						else
						{
							Span |= Value.Span;
						}
					}

					// We always drain to the left, so, have the left hand side inherit the value span
					if (!ValueStack.IsEmpty())
					{
						ValueStack.Last().Span |= Span;
					}
					break;
				}
			}
		}

		/** Push a new op */
		void PushOp(const FReduceOp& Op)
		{
			// We can only reduce up to the current frame boundary
			int32 FrameBoundary = !FrameStack.IsEmpty() ? FrameStack.Last().OpHead : 0;

			// Keep reducing until at boundary
			while (!OpStack.IsEmpty() && OpStack.Num() > FrameBoundary)
			{				
				FReduceOp& Last = OpStack.Last();

				// If less or at, we can reduce
				// TODO: Doesn't account for LHS/RHS assoc. not needed for now
				if (Last.Precedence > Op.Precedence)
				{
					break;
				}
				
				Reduce();
			}

			OpStack.Add(Op);
		}

	private:
		bool IsTrivialNonCompoundAssignment()
		{
			int32 BaseFrameAssignments = 0;
			int32 FrameCounter         = 0;
			
			for (int32 OpIndex = 0; OpIndex < OpBuffer.Num(); OpIndex++)
			{
				const FSemanticOp& Buffer = OpBuffer[OpIndex];

				// Handle scoping
				switch (Buffer.Type)
				{
					default:
					{
						break;
					}
					case ESemanticOp::FramePrologue:
					{
						FrameCounter++;
						continue;
					}
					case ESemanticOp::FrameEpilogue:
					{
						FrameCounter--;
						continue;
					}
				}

				// Ignore all nested scopes
				if (FrameCounter > 0)
				{
					continue;
				}

				// If there's anything of greater precedence, non-trivial
				if (Buffer.Precedence > EPrecedence::BinaryAssign)
				{
					return false;
				}

				// Assignment?
				BaseFrameAssignments += (Buffer.Precedence == EPrecedence::BinaryAssign);
			}

			// Ignoring nested assignment, for now
			return BaseFrameAssignments == 1;
		}

	private:
		/** Value on the stack */
		struct FStackValue
		{
			/** Optional, resolved symbol */
			FSymbol* Symbol = nullptr;

			/** If unresolved, identifer of this value */
			FSmallStringView ID{};

			/** The effectively value span this represents */
			FValueSpan Span;

			/** Should this be treated as a read site? */
			bool bIsReadSite = false;
		};

		/** Frame counters for drains */
		struct FFrameHead
		{
			int32 OpHead    = 0;
			int32 ValueHead = 0;
		};

		/** Helper, get the symbol of a stack value */
		FSymbol* ResolveValueSymbol(const FStackValue& Value) const
		{
			if (Value.Symbol)
			{
				return Value.Symbol;
			}

			if (Value.ID.Begin)
			{
				return SymCtx.FindSymbol(Value.ID);
			}

			return nullptr;
		}

		/** Helper, get the symbol of a stack value */
		FSymbol* ResolveValueSymbolReadSite(const FStackValue& Value)
		{
			if (Value.Symbol)
			{
				return Value.Symbol;
			}

			if (Value.ID.Begin)
			{
				auto Symbol = SymCtx.FindSymbol(Value.ID);

				if (Symbol)
				{
					ValueHistory.Add(FStackValue {
						.Symbol = Symbol,
						.Span = Value.Span,
						.bIsReadSite = true
					});
				}

				return Symbol;
			}

			return nullptr;
		}
		
		/** Helper, push a value to the stack */
		void PushValue(const FStackValue& Value)
		{
			ValueStack.Add(Value);
			ValueHistory.Add(Value);
		}

		/** Helper, push an unresolved value */
		void PushUnresolvedValue(const FValueSpan& Span)
		{
			PushValue(FStackValue {
				.Span = Span
			});
		}

	private:
		/** Current op buffer */
		TOpBuffer<FSemanticOp, 128> OpBuffer;

		/** Evaluation stacks */
		TArray<FStackValue, TInlineAllocator<64, TMemStackAllocator<>>> ValueStack;
		TArray<FReduceOp,   TInlineAllocator<64, TMemStackAllocator<>>> OpStack;
		TArray<FFrameHead,  TInlineAllocator<64, TMemStackAllocator<>>> FrameStack;

		/** Evaluation value history, not a stack */
		TOpBuffer<FStackValue, 1024> ValueHistory;

		/** Current statement code region */
		FParseViewType CurrentStatementRegion;

	private:
		FSymbolContext& SymCtx;
	};
}
