// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SDCECommon.h"
#include "SDCETokenizer.h"
#include "SDCEOpBuffer.h"

namespace UE::ShaderMinifier::SDCE
{
	/** Standardized precedence for all operations */
	enum class EPrecedence
	{
		None,
		Primary,
		Access,
		Declare,
		BinaryGeneric,
		BinaryAssign,
		Statement
	};

	/**
	 * As per its name, the *shallow* parser doesn't require full program understanding
	 * it instead attempts to parse what it can, and treat the rest as "complex". It's up
	 * to the visitor to interpret this as it sees fit.
	 */
	class FShallowParser
	{
	public:
		FShallowParser()
		{
			// Let's set up a standardized set of templated types. This is not complete, nor does it aim to be,
			// just to get "enough" stuff through.
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("texture")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("Texture1D")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("Texture1DArray")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("Texture1D_Array")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("Texture2D")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("Texture2DArray")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("Texture2D_Array")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("Texture2DMS")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("MS_Texture2D")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("Texture2DMSArray")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("MS_Texture2D_Array")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("Texture3D")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("TextureCube")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("TextureCubeArray")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("TextureCube_Array")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("Buffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("DataBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("AppendStructuredBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("AppendRegularBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("ByteAddressBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("ByteBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("ConsumeStructuredBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("ConsumeRegularBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RWBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RW_DataBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RWByteAddressBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RW_ByteBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RWStructuredBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RW_RegularBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RWTexture1D")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RW_Texture1D")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RWTexture1DArray")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RW_Texture1D_Array")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RWTexture2D")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RW_Texture2D")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RWTexture2DArray")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RW_Texture2D_Array")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RasterizerOrderedTexture2D")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RWTexture3D")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RW_Texture3D")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("StructuredBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RegularBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("ConstantBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("RaytracingAccelerationStructure")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("TriangleStream")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("ConstantBuffer")));
			TemplatedTypes.Add(FConstSmallStringView::Text(TEXT("vector")));
		}

		/**
		 * Parse a code chunk, visitor statically invoked
		 */
		template<typename T>
		void ParseCode(const FParseViewType& Code, T&& Visitor)
		{
			// Useful for debugging
	#if 0
			Type = ECodeChunkType::Function;
			Code = R"()";
	#endif // 0

			// Setup tokenizer
			FShallowTokenizer Tk;
			Tk.Begin = Code.Begin;
			Tk.End = Code.Begin + Code.Length;
			Tk.Next();

			// Just keep trying to parse as statements
			while (!Tk.IsEOS())
			{
				ParseStatement(Tk);
			}

			// Succeeded, propagate visitation
			for (int32 i = 0; i < OpBuffer.Num(); i++)
			{
				const FVisitOp& Op = OpBuffer[i];
				switch (Op.Type)
				{
					default:
					{
						checkNoEntry();
						break;
					}
					case EVisitOp::ID: {
						Visitor.VisitPrimaryID(Op.ID);
						break;
					}
					case EVisitOp::ComplexID: {
						Visitor.VisitComplexID(Op.ID);
						break;
					}
					case EVisitOp::Type: {
						Visitor.VisitType(Op.ID);
						break;
					}
					case EVisitOp::Binary: {
						Visitor.VisitBinary(Op.Binary.Token);
						break;
					}
					case EVisitOp::MemberAccess: {
						Visitor.VisitPrimaryMemberAccess(Op.ID);
						break;
					}
					case EVisitOp::AggregateComposite: {
						Visitor.VisitAggregateComposite();
						break;
					}
					case EVisitOp::Declaration: {
						Visitor.VisitDeclaration();
						break;
					}
					case EVisitOp::StatementPrologue: {
						Visitor.VisitStatementPrologue(Op.Statement.Cursor);
						break;
					}
					case EVisitOp::StatementEpilogue: {
						Visitor.VisitStatementEpilogue(Op.Statement.Cursor);
						break;
					}
					case EVisitOp::StructPrologue: {
						Visitor.VisitStructPrologue(Op.Struct.ID);
						break;
					}
					case EVisitOp::StructEpilogue: {
						Visitor.VisitStructEpilogue();
						break;
					}
					case EVisitOp::FunctionPrologue: {
						Visitor.VisitFunctionPrologue();
						break;
					}
					case EVisitOp::FunctionEpilogue: {
						Visitor.VisitFunctionEpilogue();
						break;
					}
					case EVisitOp::FramePrologue: {
						Visitor.VisitFramePrologue();
						break;
					}
					case EVisitOp::FrameEpilogue: {
						Visitor.VisitFrameEpilogue();
						break;
					}
					case EVisitOp::Alias: {
						Visitor.VisitAlias();
						break;
					}
					case EVisitOp::Drain: {
						Visitor.VisitDrain(Op.Drain.Type, Op.Drain.Reason);
						break;
					}
				}
			}

			OpBuffer.Empty();
			
			Tk.OpHead = TokenizerInvalidOpHead;
		}

	private:
		/** Helper, appends prologue/epilogue around a statement */
		template<typename F>
		void WithStatement(FShallowTokenizer& Tk, F&& functor)
		{
			OpBuffer.Add(FVisitOp { .Type = EVisitOp::StatementPrologue, .Statement = { .Cursor = Tk.Token.Begin } });
			
			functor();

			OpBuffer.Add(FVisitOp { .Type = EVisitOp::StatementEpilogue, .Statement = { .Cursor = Tk.Token.Begin + Tk.Token.Length } });
		}

		void ParseStruct(FShallowTokenizer& Tk)
		{
			bool bIsTemplated = SkipTemplate(Tk);
			SkipAnnotations(Tk);
			
			check(Tk.IsConsume(ETokenType::Struct));
			FToken ID = Tk.Next();

			// If templated, keep track of it
			if (bIsTemplated)
			{
				TemplatedTypes.Add(FSmallStringView::Get(ID));
			}

			OpBuffer.Add(FVisitOp { .Type = EVisitOp::StructPrologue, .Struct = { FSmallStringView::Get(ID) } });

			if (!Tk.Is(ETokenType::BodyOpen) && !Tk.Is(ETokenType::EndOfStatement))
			{
				// TODO: We need structural "skipping" with complex handlers
				Tk.SkipUntil(TEXT("{;"));
			}

			if (Tk.Is(ETokenType::BodyOpen))
			{
				ParseBody(Tk);
			}

			OpBuffer.Add(FVisitOp { .Type = EVisitOp::StructEpilogue });
		}

		bool ParseBody(FShallowTokenizer& Tk)
		{
			check(Tk.Is(ETokenType::BodyOpen));
			Tk.Next();
			
			ParseBodyInner(Tk);
			
			return Tk.IsConsume(ETokenType::BodyClose);
		}

		void ParseBodyInner(FShallowTokenizer& Tk)
		{
			while (!Tk.IsEOS() && !Tk.Is(ETokenType::BodyClose))
			{
				ParseStatement(Tk);
			}
		}

		void ParseStatement(FShallowTokenizer& Tk, bool bConsumeEOS = true)
		{
			OpBuffer.Add(FVisitOp { .Type = EVisitOp::StatementPrologue, .Statement = { .Cursor = Tk.Token.Begin } });
			
			if (!TryParseStatementInside(Tk))
			{
				OpBuffer.Add(FVisitOp { .Type = EVisitOp::StatementEpilogue, .Statement = { .Cursor = Tk.Begin } });
				SkipStatement(Tk);
				return;
			}

			OpBuffer.Add(FVisitOp { .Type = EVisitOp::StatementEpilogue, .Statement = { .Cursor = Tk.Token.Begin + Tk.Token.Length } });

			if (bConsumeEOS)
			{
				Tk.IsConsume(ETokenType::EndOfStatement);
			}
		}

		void ParseReturn(FShallowTokenizer& Tk)
		{
			int32 DrainAnchor = OpBuffer.Num();
						
			Tk.Next();

			if (!Tk.Is(ETokenType::EndOfStatement))
			{
				TryParseExpression(Tk);

				PushDrain(DrainAnchor, TEXT("Return"));
			}
		}

		void ParseDXCPragma(FShallowTokenizer& Tk)
		{
			Tk.Next();
			Tk.SkipUntil(')');
			Tk.Next();
		}

		bool ParseIf(FShallowTokenizer& Tk)
		{
			bool bFirst = true;
			
			while (bFirst || Tk.IsConsume(ETokenType::Else))
			{
				bFirst = false;
				
				if (Tk.IsConsume(ETokenType::If))
				{
					if (!Tk.IsConsume(ETokenType::ParenthesisOpen))
					{
						return false;
					}

					WithStatement(Tk, [&] { TryParseExpression(Tk); });

					if (!Tk.Is(ETokenType::ParenthesisClose))
					{
						return false;
					}
			
					Tk.Next();
				}

				ParseStatementBody(Tk);
			}

			return true;
		}

		bool ParseWhile(FShallowTokenizer& Tk)
		{
			Tk.Next();
			
			if (!Tk.IsConsume(ETokenType::ParenthesisOpen))
			{
				return false;
			}

			WithStatement(Tk, [&] { TryParseExpression(Tk); });

			if (!Tk.IsConsume(ETokenType::ParenthesisClose))
			{
				return false;
			}

			ParseStatementBody(Tk);
			return true;
		}

		bool ParseFor(FShallowTokenizer& Tk)
		{
			Tk.Next();

			if (!Tk.IsConsume(ETokenType::ParenthesisOpen))
			{
				return false;
			}

			// Initializer
			if (!Tk.IsConsume(ETokenType::EndOfStatement))
			{
				ParseStatement(Tk, false);
				
				if (!Tk.IsConsume(ETokenType::EndOfStatement))
				{
					return false;
				}
			}

			// Condition
			if (!Tk.IsConsume(ETokenType::EndOfStatement))
			{
				ParseStatement(Tk, false);

				if (!Tk.IsConsume(ETokenType::EndOfStatement))
				{
					return false;
				}
			}

			// Step
			if (!Tk.Is(ETokenType::ParenthesisClose))
			{
				ParseStatement(Tk);
			}

			if (!Tk.IsConsume(ETokenType::ParenthesisClose))
			{
				return false;
			}

			ParseStatementBody(Tk);

			return true;
		}

		void ParseStatementBody(FShallowTokenizer& Tk)
		{
			if (Tk.Is(ETokenType::BodyOpen))
			{
				ParseBody(Tk);
			}
			else
			{
				ParseStatement(Tk);
			}
		}

		bool ParseSwitch(FShallowTokenizer& Tk)
		{
			Tk.Next();

			if (!Tk.IsConsume(ETokenType::ParenthesisOpen))
			{
				return false;
			}

			int32 DrainAnchor = OpBuffer.Num();
						
			TryParseExpression(Tk);
			
			PushDrain(DrainAnchor, TEXT("Switch-Cond"));

			if(!Tk.IsConsume(ETokenType::ParenthesisClose))
			{
				return false;
			}

			if(!Tk.IsConsume(ETokenType::BodyOpen))
			{
				return false;
			}

			while (!Tk.IsEOS() && !Tk.Is(ETokenType::BodyClose))
			{
				if (Tk.IsConsume(ETokenType::Default))
				{
					
				}
				else if (Tk.IsConsume(ETokenType::Case))
				{
					if (!TryParsePrimary(Tk))
					{
						break;
					}
			
					PushDrain(DrainAnchor, TEXT("Switch-Case"));
				}
				else
				{
					break;
				}

				if (!Tk.IsConsume(ETokenType::Colon))
				{
					break;
				}

				// Fallthrough?
				if (Tk.Is(ETokenType::Case))
				{
					continue;
				}

				while (!Tk.Is(ETokenType::BodyClose) && !Tk.Is(ETokenType::Default) && !Tk.Is(ETokenType::Case))
				{
					ParseStatement(Tk);
				}
			}

			return Tk.IsConsume(ETokenType::BodyClose);
		}

		void ParseBreak(FShallowTokenizer& Tk)
		{
			Tk.Next();
		}

		bool TryParseStatementInside(FShallowTokenizer& Tk)
		{
			// For now, just ignore all annotations
			SkipAnnotations(Tk);

			switch (Tk.Token.Type)
			{
				default:
				{
					break;
				}
				case ETokenType::Return:
				{
					ParseReturn(Tk);
					return true;
				}
				case ETokenType::If:
				{
					return ParseIf(Tk);
				}
				case ETokenType::DXCPragma:
				{
					ParseDXCPragma(Tk);
					return true;
				}
				case ETokenType::While:
				{
					return ParseWhile(Tk);
				}
				case ETokenType::Switch:
				{
					return ParseSwitch(Tk);
				}
				case ETokenType::For:
				{
					return ParseFor(Tk);
				}
				case ETokenType::Struct:
				{
					ParseStruct(Tk);
					return true;
				}
				case ETokenType::BodyOpen:
				{
					return ParseBody(Tk);
				}
				case ETokenType::Break:
				{
					ParseBreak(Tk);
					return true;
				}
			}
			
			if (TryParseFunction(Tk))
			{
				return true;
			}
			
			if (TryParseDeclaration(Tk))
			{
				return true;
			}
			
			if (TryParseExpression(Tk))
			{
				return true;
			}

			return false;
		}

		void SkipStatement(FShallowTokenizer& Tk)
		{
			if (!Tk.IsConsume(ETokenType::EndOfStatement))
			{
				// TODO: Structural skipping!
				FParseCharType* Begin = Tk.Begin;
				Tk.SkipUntil(';');

				// Mark the full range as complex
				MarkIdentifiersAsComplexBase(FParseViewType(Begin, Tk.Begin - Begin));

	#if UE_SHADER_SDCE_LOG_COMPLEX
				UE_LOGF(LogTemp, Error, "Skipped complex statement: %s", *FParseStringType::ConstructFromPtrSize(Begin, Tk.Begin - Begin));
	#endif // UE_SHADER_SDCE_LOG_COMPLEX
			}
		}

		void MarkIdentifiersAsComplexBase(const FParseViewType& Region)
		{
			// Setup tokenizer
			FShallowTokenizer Tk;
			Tk.Begin = Region.Begin;
			Tk.End = Region.Begin + Region.Length;
			Tk.Next();

			// Mark all identifiers as complex
			while (!Tk.IsEOS())
			{
				if (Tk.Is(ETokenType::ID))
				{
					OpBuffer.Add(FVisitOp { .Type = EVisitOp::ComplexID, .ID = FSmallStringView::Get(Tk.Token) });
				}
				
				Tk.Next();
			}
			
			Tk.OpHead = TokenizerInvalidOpHead;
		}
		
		bool TryParseFunction(FShallowTokenizer& TkOuter)
		{
			FShallowTokenizer Tk = Branch(TkOuter);

			SkipTemplate(Tk);
			SkipAnnotations(Tk);

			FToken ReturnType;
			if (!TryParseType(Tk, ReturnType))
			{
				Reject(Tk);
				return false;
			}

			if (!TryParsePrimary(Tk, false))
			{
				Reject(Tk);
				return false;
			}

			if (!Tk.IsConsume(ETokenType::ParenthesisOpen))
			{
				Reject(Tk);
				return false;
			}

			OpBuffer.Add(FVisitOp { .Type = EVisitOp::FunctionPrologue });

			// Parse all parameters
			while (!Tk.IsEOS() && !Tk.Is(ETokenType::ParenthesisClose))
			{
				SkipQualifiers(Tk);

				WithStatement(Tk, [&] { TryParseDeclaration(Tk, false); });

				if (!Tk.IsConsume(ETokenType::Comma))
				{
					break;
				}
			}

			if (!Tk.IsConsume(ETokenType::ParenthesisClose))
			{
				// TODO: Maybe this is less of a matching failure, and more of a chunk failure?
				Reject(Tk);
				return false;
			}

			if (Tk.Is(ETokenType::BodyOpen))
			{
				ParseBody(Tk);
			}

			OpBuffer.Add(FVisitOp { .Type = EVisitOp::FunctionEpilogue });

			Accept(TkOuter, Tk);
			return true;
		}

		bool TryParseDeclaration(FShallowTokenizer& TkOuter, bool bAllowMultiDeclarators = true)
		{
			FShallowTokenizer Tk = Branch(TkOuter);

			SkipTemplate(Tk);
			SkipQualifiers(Tk);

			int32 TypeStart = OpBuffer.Num();
			
			FToken Type;
			if (!TryParseType(Tk, Type))
			{
				Reject(Tk);
				return false;
			}

			int32 TypeEnd = OpBuffer.Num();

			// May have multiple declarators "int a, b, ..."
			for (;;)
			{
				int32 DrainAnchor = OpBuffer.Num();

				// Push type ops again if secondary
				if (OpBuffer.Num() != TypeEnd)
				{
					for (int i = TypeStart; i < TypeEnd; ++i)
					{
						OpBuffer.Add(OpBuffer[i]);
					}
				}
				
				FToken ID = Tk.Next();
				if (ID.Type != ETokenType::ID)
				{
					Reject(Tk);
					return false;
				}

				OpBuffer.Add(FVisitOp { .Type = EVisitOp::ID, .ID = FSmallStringView::Get(ID) });

				// Declarators may have type postfixes after the ID
				TryParseTypePostfix(Tk);

				// Don't care about bindings
				SkipDeclarationBindings(Tk);

				OpBuffer.Add(FVisitOp { .Type = EVisitOp::Declaration });

				// Handle inline assignments
				if (Tk.Is(ETokenType::BinaryEq))
				{
					OpBuffer.Add(FVisitOp { .Type = EVisitOp::Binary, .Binary = { Tk.Token } });

					Tk.Next();
				
					if (!TryParseExpression(Tk, false))
					{
						Reject(Tk);
						return false;
					}
				}

				if (!bAllowMultiDeclarators || !Tk.IsConsume(ETokenType::Comma))
				{
					break;
				}

				// Last value carries over
				PushDrain(DrainAnchor, TEXT("MultiDeclarator"));
			}

			Accept(TkOuter, Tk);
			return true;
		}

		bool TryParseType(FShallowTokenizer& Tk, FToken& Type)
		{
			// Note: This is obviously an over simplified type
			// When needed, it'll be expanded on
			
			Type = Tk.Next();
			if (Type.Type != ETokenType::ID)
			{
				return false;
			}

			OpBuffer.Add(FVisitOp { .Type = EVisitOp::Type, .ID = { FSmallStringView::Get(Type) } });

			TryParseTypePostfix(Tk);

			if (TemplatedTypes.Contains(FSmallStringView::Get(Type)))
			{
				SkipTemplateSpecialization(Tk);
			}

			return true;
		}

		void TryParseTypePostfix(FShallowTokenizer& TkOuter)
		{
			FShallowTokenizer Tk = Branch(TkOuter);
			
			for (;;)
			{
				if (Tk.IsConsume(ETokenType::SquareOpen))
				{
					// Reject annotations
					if (Tk.Is(ETokenType::SquareOpen))
					{
						// No longer a type postfix, any prior postfixes have been accepted at this point
						Reject(Tk);
						break;
					}
					
					// May close immediately for automatic sizing
					if (!Tk.Is(ETokenType::SquareClose))
					{
						int32 DrainAnchor = OpBuffer.Num();
						TryParseExpression(Tk);
						PushDrain(DrainAnchor, TEXT("TypeArrayDim"));
					}
					
					check(Tk.Is(ETokenType::SquareClose));
					Tk.Next();
					
					Accept(TkOuter, Tk);
					Tk = Branch(TkOuter);
					continue;
				}
				
				// Not a postfix
				Reject(Tk);
				break;
			}
		}

		bool IsUnaryPrimaryValueModifier(ETokenType Type)
		{
			switch (Type)
			{
			default:
				return false;
				case ETokenType::BinaryAdd:
				case ETokenType::BinarySub:
				case ETokenType::UnaryBitNegate:
				case ETokenType::UnaryNot:
				case ETokenType::UnaryInc:
				case ETokenType::UnaryDec:
					return true;
			}
		}

		bool ParseParenthesis(FShallowTokenizer& Tk)
		{
			Tk.Next();

			OpBuffer.Add(FVisitOp { .Type = EVisitOp::FramePrologue, .bPrivate = false });
			
			int32 DrainAnchor = OpBuffer.Num();
			TryParseExpression(Tk);

			// Expecting close
			if (!Tk.IsConsume(ETokenType::ParenthesisClose))
			{
				return false;
			}

			// Cast?
			if (Tk.Is(ETokenType::ID) || Tk.Is(ETokenType::Numeric) || IsUnaryPrimaryValueModifier(Tk.Token.Type))
			{
				PushDrain(DrainAnchor, TEXT("Cast"));

				OpBuffer.Add(FVisitOp { .Type = EVisitOp::FrameEpilogue, .bPrivate = false });
				
				return TryParsePrimary(Tk);
			}
			
			OpBuffer.Add(FVisitOp { .Type = EVisitOp::FrameEpilogue, .bPrivate = false });

			// Otherwise, allow accessors
			TryParsePrimaryAccessors(Tk);

			return true;
		}

		void ParseTernary(FShallowTokenizer& Tk, bool bAllowExpressionCarry = true)
		{
			check(Tk.Is(ETokenType::Ternary));
			Tk.Next();

			// Pass value
			{
				OpBuffer.Add(FVisitOp { .Type = EVisitOp::FramePrologue, .bPrivate = false });
			
				int32 DrainAnchor = OpBuffer.Num();
				TryParseExpression(Tk, bAllowExpressionCarry);
				PushDrain(DrainAnchor, TEXT("TenaryPass"));
			
				OpBuffer.Add(FVisitOp { .Type = EVisitOp::FrameEpilogue, .bPrivate = false });
			}

			check(Tk.Is(ETokenType::Colon));
			Tk.Next();

			// Failure value
			{
				OpBuffer.Add(FVisitOp { .Type = EVisitOp::FramePrologue, .bPrivate = false });
			
				int32 DrainAnchor = OpBuffer.Num();
				TryParseExpression(Tk, bAllowExpressionCarry);
				PushDrain(DrainAnchor, TEXT("TernaryFail"));
			
				OpBuffer.Add(FVisitOp { .Type = EVisitOp::FrameEpilogue, .bPrivate = false });
			}
		}

		bool TryParseExpression(FShallowTokenizer& TkOuter, bool bAllowExpressionCarry = true)
		{
			FShallowTokenizer Tk = Branch(TkOuter);

			int32 DrainAnchor = OpBuffer.Num();

			if (!TryParsePrimary(Tk))
			{
				Reject(Tk);
				return false;
			}

			if (!TryParseBinary(Tk))
			{
				Reject(Tk);
				return false;
			}

			if (Tk.Is(ETokenType::Ternary))
			{
				ParseTernary(Tk, bAllowExpressionCarry);
			}

			if (bAllowExpressionCarry && Tk.IsConsume(ETokenType::Comma))
			{
				PushDrain(DrainAnchor, TEXT("CommaCarry"));

				if (!TryParseExpression(Tk))
				{
					Reject(Tk);
					return false;
				}
			}
			
			Accept(TkOuter, Tk);
			return true;
		}

		bool ParseUnary(FShallowTokenizer& Tk)
		{
			Tk.Next();
			return TryParsePrimary(Tk);
		}

		bool ParseAggregateInitializer(FShallowTokenizer& Tk)
		{
			Tk.Next();

			while (!Tk.IsEOS() && !Tk.Is(ETokenType::BodyClose))
			{
				int32 DrainAnchor = OpBuffer.Num();
				
				if (!TryParseExpression(Tk, false))
				{
					return false;
				}
		
				PushDrain(DrainAnchor, TEXT("AggregateInitMember"));

				if (!Tk.IsConsume(ETokenType::Comma))
				{
					break;
				}
			}

			// Zero for now, until we need it
			OpBuffer.Add(FVisitOp { .Type = EVisitOp::AggregateComposite });
					
			return Tk.IsConsume(ETokenType::BodyClose);
		}

		bool TryParsePrimary(FShallowTokenizer& TkOuter, bool bAllowCall = true)
		{
			if (TkOuter.Is(ETokenType::ParenthesisOpen))
			{
				return ParseParenthesis(TkOuter);
			}

			if (IsUnaryPrimaryValueModifier(TkOuter.Token.Type))
			{
				return ParseUnary(TkOuter);
			}

			if (TkOuter.Is(ETokenType::BodyOpen))
			{
				return ParseAggregateInitializer(TkOuter);
			}

			FShallowTokenizer Tk = Branch(TkOuter);
			
			FToken PrimID = Tk.Next();
			if (PrimID.Type != ETokenType::ID && PrimID.Type != ETokenType::Numeric)
			{
				Reject(Tk);
				return false;
			}

			OpBuffer.Add(FVisitOp { .Type = EVisitOp::ID, .ID = { FSmallStringView::Get(PrimID) } });
			
			if (!TryParsePrimaryAccessors(Tk, bAllowCall))
			{
				Reject(Tk);
				return false;
			}

			Accept(TkOuter, Tk);
			return true;
		}

		void PushDrain(int32 Anchor, const TCHAR* Reason)
		{
			// TODO: Ignore drains on statement ends
			EPrecedence Precedence = EPrecedence::None;

			// Zero op range? early out
			if (Anchor == OpBuffer.Num()) 
			{
				return;
			}

			// Current number of private frames
			int32 PrivateFrameCounter = 0;

			// We need to maintain the precedence of the value that we're draining,
			// otherwise we violate the op "hierarchy" (it's flat, but the implicit one).
			for (int32 i = Anchor; i < OpBuffer.Num(); i++)
			{
				EPrecedence VisitPrecedence;

				// Get precedence for the op
				const FVisitOp& Op = OpBuffer[i];
				switch (Op.Type)
				{
					default:
					{
						checkNoEntry();
						continue;
					}
					case EVisitOp::ID:
					{
						VisitPrecedence = EPrecedence::Primary;
						break;
					}
					case EVisitOp::Type:
					{
						VisitPrecedence = EPrecedence::Primary;
						break;
					}
					case EVisitOp::Binary:
					{
						VisitPrecedence = Op.Binary.Token.Type == ETokenType::BinaryEq ? EPrecedence::BinaryAssign : EPrecedence::BinaryGeneric;
						break;
					}
					case EVisitOp::MemberAccess: {
							
						VisitPrecedence = EPrecedence::Access;
						break;
					}
					case EVisitOp::Declaration:
					{
						VisitPrecedence = EPrecedence::Declare;
						break;
					}
					case EVisitOp::AggregateComposite:
					{
						VisitPrecedence = EPrecedence::Primary;
						break;
					}
					case EVisitOp::FramePrologue:
					{
						if (Op.bPrivate)
						{
							PrivateFrameCounter++;
						}
						continue;
					}
					case EVisitOp::FrameEpilogue:
					{
						if (Op.bPrivate)
						{
							PrivateFrameCounter--;
						}
						continue;
					}
					case EVisitOp::StatementPrologue:
					case EVisitOp::StatementEpilogue:
					case EVisitOp::StructPrologue:
					case EVisitOp::StructEpilogue:
					case EVisitOp::FunctionPrologue:
					case EVisitOp::FunctionEpilogue:
					case EVisitOp::Alias: 
					case EVisitOp::Drain:
					{
						VisitPrecedence = EPrecedence::None;
						break;
					}
				}

				// If we're in a private frame, do not account for it
				if (PrivateFrameCounter > 0)
				{
					continue;
				}

				Precedence = FMath::Max(Precedence, VisitPrecedence);
			}

			// Must close out
			check(PrivateFrameCounter == 0);

			// Must have found at least one op
			check(Precedence != EPrecedence::None);

			OpBuffer.Add(FVisitOp { .Type = EVisitOp::Drain, .Drain = { Reason, Precedence } });
		}

		bool TryParsePrimaryAccessors(FShallowTokenizer& Tk, bool bAllowCall = true)
		{
			for (;;)
			{
				// ++/--
				if (Tk.Is(ETokenType::UnaryInc) || Tk.Is(ETokenType::UnaryDec))
				{
					Tk.Next();
					continue;
				}

				// Array accessor
				if (Tk.Is(ETokenType::SquareOpen))
				{
					Tk.Next();
					
					int32 DrainAnchor = OpBuffer.Num();
						
					if (!TryParseExpression(Tk))
					{
						return false;
					}

					PushDrain(DrainAnchor, TEXT("ArrayAccessor"));

					check(Tk.Is(ETokenType::SquareClose));
					Tk.Next();
					
					continue;
				}

				// Member access
				if (Tk.Is(ETokenType::Access) || Tk.Is(ETokenType::NamespaceAccess))
				{
					Tk.Next();
					
					FToken ID = Tk.Next();
					if (ID.Type != ETokenType::ID)
					{
						return false;
					}

					OpBuffer.Add(FVisitOp { .Type = EVisitOp::MemberAccess, .ID = { FSmallStringView::Get(ID) } });
					
					continue;
				}
				
				// Calls
				if (bAllowCall && Tk.Is(ETokenType::ParenthesisOpen))
				{
					Tk.Next();

					while (!Tk.IsEOS() && !Tk.Is(ETokenType::ParenthesisClose))
					{
						OpBuffer.Add(FVisitOp { .Type = EVisitOp::FramePrologue, .bPrivate = true });
					
						int32 DrainAnchor = OpBuffer.Num();
						
						if (!TryParseExpression(Tk, false))
						{
							return false;
						}

						PushDrain(DrainAnchor, TEXT("CallArg"));
						
						OpBuffer.Add(FVisitOp { .Type = EVisitOp::FrameEpilogue, .bPrivate = true });

						if (!Tk.IsConsume(ETokenType::Comma))
						{
							break;
						}
					}

					// Unexpected closure
					if (!Tk.Is(ETokenType::ParenthesisClose))
					{
						return false;
					}
					
					Tk.Next();
					continue;
				}

				// Not an accessor
				break;
			}

			// OK
			return true;
		}

		bool TryParseBinary(FShallowTokenizer& Tk)
		{
			// Note: Other than assignments, we are effectively not caring about precedence
			
			while (!Tk.IsEOS())
			{
				// Is this a binary operation?
				switch (Tk.Token.Type)
				{
				default:
					// OK
					return true;
				case ETokenType::BinaryEq:
				case ETokenType::BinaryNotEq:
				case ETokenType::BinaryPlusEq:
				case ETokenType::BinarySubEq:
				case ETokenType::BinaryDivEq:
				case ETokenType::BinaryMulEq:
				case ETokenType::BinaryModEq:
				case ETokenType::BinaryXEq:
				case ETokenType::BinaryOrEq:
				case ETokenType::BinaryAndEq:
				case ETokenType::BinaryShlEq:
				case ETokenType::BinaryShrEq:
				case ETokenType::LogicalAnd:
				case ETokenType::LogicalOr:
				case ETokenType::BinaryLess:
				case ETokenType::BinaryLessEq:
				case ETokenType::BinaryGreater:
				case ETokenType::BinaryGreaterEq:
				case ETokenType::BinaryLogicalEq:
				case ETokenType::BinaryAdd:
				case ETokenType::BinarySub:
				case ETokenType::BinaryDiv:
				case ETokenType::BinaryMul:
				case ETokenType::BinaryMod:
				case ETokenType::BinaryXor:
				case ETokenType::BinaryBitOr:
				case ETokenType::BinaryBitAnd:
				case ETokenType::BinaryShl:
				case ETokenType::BinaryShr:
					break;
				}

				OpBuffer.Add(FVisitOp { .Type = EVisitOp::Binary, .Binary = { Tk.Token } });
				
				Tk.Next();

				if (!TryParsePrimary(Tk))
				{
					return false;
				}
			}

			return true;
		}

		bool TryParseUsingAlias(FShallowTokenizer& Tk)
		{
			check(Tk.Is(ETokenType::Using));
			Tk.Next();

			FToken ID = Tk.Next();
			if (ID.Type != ETokenType::ID)
			{
				return false;
			}

			if (!Tk.IsConsume(ETokenType::BinaryEq))
			{
				return false;
			}
			
			FToken Type;
			if (!TryParseType(Tk, Type))
			{
				return false;
			}

			OpBuffer.Add(FVisitOp { .Type = EVisitOp::Alias });

			return true;
		}

		bool TryParseTypedefAlias(FShallowTokenizer& Tk)
		{
			check(Tk.Is(ETokenType::Typedef));
			Tk.Next();

			FToken Type;
			if (!TryParseType(Tk, Type))
			{
				return false;
			}
			
			FToken ID = Tk.Next();
			if (ID.Type != ETokenType::ID)
			{
				return false;
			}

			OpBuffer.Add(FVisitOp { .Type = EVisitOp::Alias });

			return true;
		}

		void SkipAnnotations(FShallowTokenizer& Tk)
		{
			while (Tk.Is(ETokenType::SquareOpen))
			{
				// TODO: Structural skipping!
				Tk.SkipUntil(TEXT(']'));
				Tk.Next();
			}
		}

		bool SkipTemplate(FShallowTokenizer& Tk)
		{
			if (!Tk.IsConsume(ETokenType::Template))
			{
				return false;
			}

			if (!Tk.Is(ETokenType::BinaryGreater))
			{
				// TODO: Structural skipping!
				Tk.SkipUntil(TEXT('>'));
			}
			
			Tk.Next();
			return true;
		}

		void SkipTemplateSpecialization(FShallowTokenizer& Tk)
		{
			if (!Tk.IsConsume(ETokenType::BinaryLess))
			{
				return;
			}

			if (!Tk.Is(ETokenType::BinaryGreater))
			{
				// TODO: Structural skipping!
				Tk.SkipUntil(TEXT('>'));
			}
			
			Tk.Next();
		}

		void SkipDeclarationBindings(FShallowTokenizer& Tk)
		{
			if (!Tk.IsConsume(ETokenType::Colon))
			{
				return;
			}

			// If this is a binding, make an educated guess
			if (!Tk.Is(ETokenType::Comma) && !Tk.Is(ETokenType::EndOfStatement) && !Tk.Is(ETokenType::ParenthesisClose))
			{
				Tk.SkipUntil(TEXT(",;"));
			}
		}

		void SkipQualifiers(FShallowTokenizer& Tk)
		{
			for (;;)
			{
				switch (Tk.Token.Type)
				{
				default:
					return;
				case ETokenType::Static:
				case ETokenType::Const:
				case ETokenType::In:
				case ETokenType::Out:
				case ETokenType::Inout:
				case ETokenType::NoInterp:
				case ETokenType::NoInterpolation:
				case ETokenType::NoPerspective:
				case ETokenType::GroupShared:
				case ETokenType::Triangle:
				case ETokenType::Centroid:
				case ETokenType::Vertices:
				case ETokenType::Indices:
				case ETokenType::Primitives:
				case ETokenType::Uniform:
					Tk.Next();
					break;
				}
			}
		}

	private:
		FShallowTokenizer Branch(const FShallowTokenizer& Tk)
		{
			FShallowTokenizer Branch = Tk.Branch();
			Branch.OpHead = OpBuffer.Num();
			return MoveTemp(Branch);
		}

		void Reject(FShallowTokenizer& Tk)
		{
			OpBuffer.SetNumNoAlloc(Tk.OpHead);
			Tk.OpHead = TokenizerInvalidOpHead;
		}

		void Accept(FShallowTokenizer& TkOuter, FShallowTokenizer& Tk)
		{
			TkOuter.Accept(Tk);
		}

	private:
		enum class EVisitOp
		{
			ID,
			ComplexID,
			Type,
			Binary,
			MemberAccess,
			AggregateComposite,
			Declaration,
			StatementPrologue,
			StatementEpilogue,
			StructPrologue,
			StructEpilogue,
			FunctionPrologue,
			FunctionEpilogue,
			FramePrologue,
			FrameEpilogue,
			Alias,
			Drain
		};

		struct FVisitOp
		{
			EVisitOp Type;

			/** Payload */
			union
			{
				/** EVisitOp::ID/ComplexID */
				FSmallStringView ID;

				/** EVisitOp::FramePrologue/FrameEpilogue */
				bool bPrivate;
				
				/** EVisitOp::StructPrologue */
				struct
				{
					FSmallStringView ID;
				} Struct;
				
				/** EVisitOp::Binary */
				struct
				{
					FToken Token;
				} Binary;

				/** EVisitOp::Drain */
				struct
				{
					const TCHAR* Reason;
					EPrecedence Type;
				} Drain;
				
				/** EVisitOp::StatementPrologue/StatementEpilogue */
				struct
				{
					FParseCharType* Cursor;
				} Statement;
			};
		};

		static_assert(sizeof(FVisitOp) == 24, "Unexpected FVisitOp size");

		/** Current visitation buffer */
		TOpBuffer<FVisitOp, 256> OpBuffer;

	private:
		/** Tracked templated types */
		TSet<FConstSmallStringView, FConstSmallStringKeyFuncs, FMemStackSetAllocator> TemplatedTypes;
	};
}
