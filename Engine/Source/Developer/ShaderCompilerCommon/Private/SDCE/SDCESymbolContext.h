// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SDCECommon.h"
#include "SDCEToken.h"
#include "Containers/ChunkedArray.h"

namespace UE::ShaderMinifier::SDCE
{
	enum class ESymbolKind
	{
		/** Generic composite type, e.g., structs */
		Composite,

		/** Declaration symbols */
		Variable
	};
		
	struct FSymbol
	{
		/** Kind of this symbol */
		ESymbolKind Kind;

		/** Name of this symbol */
		FSmallStringView ID{};

		/** Is this symbol tagged? i.e., can we perform SDCE on it? */
		bool bTagged = false;
	
		struct
		{
			/** Type of this variable */
			FSymbol* Type = nullptr;

			/** Number of realized loads, zero implies safe DCE'ing */
			int32 RealizedLoadCount = 0;
		} Variable;
	
		struct
		{
			/**
			 * All symbols within
			 * TODO: I really, really don't like bloating all symbols like this
			 */
			TArray<FSymbol*, TMemStackAllocator<>> ContainedSymbols;
		} Composite;
	};
	
	struct FSymbolScope
	{
		/** Owning composite of this scope */
		FSymbol* Composite = nullptr;

		/** All declared symbols within this scope */
		TMap<FSmallStringView, FSymbol*,  FMemStackSetAllocator, FSmallStringKeyFuncs> Symbols;
	};
	
	/** Value span, represents a range of values */
	struct FValueSpan
	{
		FValueSpan() : FValueSpan(0, 0)
		{
			
		}
	
		FValueSpan(int32 InBegin, int32 InEnd)
		{
			Begin = InBegin;
			End = InEnd;
		}
				
		FValueSpan operator|(const FValueSpan& RHS) const
		{
			FValueSpan Span;
			Span.Begin = FMath::Min(Begin, RHS.Begin);
			Span.End = FMath::Max(End, RHS.End);
			return Span;
		}
				
		FValueSpan& operator|=(const FValueSpan& RHS)
		{
			Begin = FMath::Min(Begin, RHS.Begin);
			End = FMath::Max(End, RHS.End);
			return *this;
		}
				
		int32 Begin;
		int32 End;
	};

	struct FMemberStoreSite
	{
		/** Symbols */
		FSymbol* AddressSymbol = nullptr;
		FSymbol* ValueSymbol   = nullptr;

		/** Value spans */
		FValueSpan AddressSpan;
		FValueSpan ValueSpan;

		/** Has this site been visited? Used for iterative elimination */
		bool bVisited = false;

		/** Was this determined to be a complex site? */
		bool bIsResolvedToComplex = false;

		/** Textual site */
		FSmallStringView AssignmentSite;
	};
	
	class FSymbolContext
	{
	public:
		FSymbolContext()
		{
			// Root stack
			SymbolStack.Emplace();
		}

		/** Find a symbol within a composite */
		FSymbol* LinearCompositeFindSymbol(FSymbol* Base, const FSmallStringView& View)
		{
			if (Base->Kind != ESymbolKind::Composite)
			{
				return nullptr;
			}
			
			for (FSymbol* CandidateSymbol : Base->Composite.ContainedSymbols)
			{
				if (CandidateSymbol->ID == View)
				{
					return CandidateSymbol;
				}
			}
	
			return nullptr;
		}

		/** Find a symbol in the current scope */
		FSymbol* FindSymbol(const FSmallStringView& View)
		{
			if (!View.Begin)
			{
				return nullptr;
			}
			
			// Allow resolving to the first parent aggregate symbol scope
			// Anything beyond that is ill-defined
			bool bIsImmediateFirstAggregate = true;
			
			// Try inner to outer
			for (int32 i = SymbolStack.Num() - 1; i >= 0; i--)
			{
				FSymbolScope& Scope = SymbolStack[i];
				
				if (Scope.Composite)
				{
					if (!bIsImmediateFirstAggregate)
					{
						// This is a private scope
						continue;
					}
	
					// We are now in "this"
					bIsImmediateFirstAggregate = false;
				}
				
				if (FSymbol** It = Scope.Symbols.Find(View))
				{
					return *It;
				}
			}
	
			return nullptr;
		}

		/** Create a new symbol */
		FSymbol* CreateSymbol(ESymbolKind Kind)
		{
			FSymbol* Symbol = &Symbols[Symbols.Emplace()];
			Symbol->Kind = Kind;
			return Symbol;
		}

		/** Add a complex accessor to the set */
		void AddComplexAccessor(FSmallStringView ID)
		{
#if UE_SHADER_SDCE_LOG_COMPLEX
			UE_LOGF(LogTemp, Error, "Added complex symbol: %s", *FAnsiString::ConstructFromPtrSize(ID.Begin, ID.Length));
#endif // UE_SHADER_SDCE_LOG_COMPLEX
	
			if (ID.Begin)
			{
				ComplexBaseSet.Add(ID);
			}
		}

		/** All symbol stacks */
		TArray<FSymbolScope, TMemStackAllocator<>> SymbolStack;

		/** All accessors that we're treating as complex */
		TSet<FSmallStringView, FSmallStringKeyFuncs, FMemStackSetAllocator> ComplexBaseSet;

		/** All tracked store sites */
		TArray<FMemberStoreSite, TMemStackAllocator<>> StoreSites;
		
	private:
		/** Symbol container */
		TChunkedArray<FSymbol, 256, TMemStackAllocator<>> Symbols;
	};

	static_assert(sizeof(FSymbol) == 64, "Unexpected FSymbol packing");
}
