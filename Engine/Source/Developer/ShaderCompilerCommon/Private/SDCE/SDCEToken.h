// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SDCECommon.h"
#include "Containers/SetUtilities.h"
#include "Hash/xxhash.h"

namespace UE::ShaderMinifier::SDCE
{
	/** Full keyword/segment table */
	#define UE_MINIFIER_SDCE_KEYWORD_TABLE \
		X(If, "if") \
		X(Else, "else") \
		X(Switch, "switch") \
		X(Case, "case") \
		X(Default, "default") \
		X(For, "for") \
		X(While, "while") \
		X(Do, "do") \
		X(Break, "break") \
		X(Continue, "continue") \
		X(Discard, "discard") \
		X(Return, "return") \
		X(Struct, "struct")  \
		X(Namespace, "namespace") \
		X(BinaryEq, "=") \
		X(BinaryNotEq, "!=") \
		X(BinaryPlusEq, "+=") \
		X(BinarySubEq, "-=") \
		X(BinaryDivEq, "/=") \
		X(BinaryMulEq, "*=") \
		X(BinaryModEq, "%=") \
		X(BinaryXEq, "^=") \
		X(BinaryOrEq, "|=") \
		X(BinaryAndEq, "&=") \
		X(BinaryShlEq, "<<=") \
		X(BinaryShrEq, ">>=") \
		X(LogicalAnd, "&&") \
		X(LogicalOr, "||") \
		X(BinaryLess, "<") \
		X(BinaryLessEq, "<=") \
		X(BinaryGreater, ">") \
		X(BinaryGreaterEq, ">=") \
		X(BinaryLogicalEq, "==") \
		X(BinaryAdd, "+") \
		X(BinarySub, "-") \
		X(BinaryDiv, "/") \
		X(BinaryMul, "*") \
		X(BinaryMod, "%") \
		X(BinaryXor, "^") \
		X(BinaryBitOr, "|") \
		X(BinaryBitAnd, "&") \
		X(BinaryShl, "<<") \
		X(BinaryShr, ">>") \
		X(BodyOpen, "{") \
		X(BodyClose, "}") \
		X(ParenthesisOpen, "(") \
		X(ParenthesisClose, ")") \
		X(SquareOpen, "[") \
		X(SquareClose, "]") \
		X(EndOfStatement, ";") \
		X(Ternary, "?") \
		X(Comma, ",") \
		X(Colon, ":") \
		X(Access, ".") \
		X(NamespaceAccess, "::") \
		X(UnaryBitNegate, "~") \
		X(UnaryNot, "!") \
		X(UnaryInc, "++") \
		X(UnaryDec, "--") \
		X(Using, "using") \
		X(Typedef, "typedef") \
		X(Template, "template") \
		X(Static, "static") \
		X(Const, "const") \
		X(In, "in") \
		X(Out, "out") \
		X(Inout, "inout") \
		X(NoInterp, "nointerp") \
		X(NoInterpolation, "nointerpolation") \
		X(NoPerspective, "noperspective") \
		X(GroupShared, "groupshared") \
		X(Triangle, "triangle") \
		X(Vertices, "vertices") \
		X(Indices, "indices") \
		X(Primitives, "primitives") \
		X(Uniform, "uniform") \
		X(Centroid, "centroid") \
		X(DXCPragma, "_Pragma")

	enum class ETokenType : uint16_t
	{
		/** Standard identifier */
		ID,

		/** Numeric identifier */
		Numeric,

		/** Unknown token */
		Unknown,

		/** Segments */
#define X(N, S) N,
		UE_MINIFIER_SDCE_KEYWORD_TABLE
#undef X
	};
		
	struct FToken
	{
		/** Set from an identifier */
		void SetIDNoHash(const FParseViewType& View)
		{
			Begin = View.Begin;
			Length = static_cast<uint16_t>(View.Length);
			XXHash32 = 0;
		}

		// DW0
		ETokenType Type;
		uint16_t Length;
		// DW1
		// Each token inlines the identifier hash, this is faster than doing it on demand
		uint32_t XXHash32;
		// DW2-4
		FParseCharType* Begin;
	};
	
	struct FSmallStringView
	{
		/** Get from a token  */
		static FSmallStringView Get(const FToken& Token)
		{
			return FSmallStringView {
				Token.Begin,
				Token.XXHash32,
				Token.Length
			};
		}

		/** Hash a text region */
		static uint32_t Hash(const FParseCharType* Data, uint16 Length)
		{
			// TODO: Mod?
			// TODO: Hash only the ansi bits?
			return static_cast<uint32_t>(FXxHash64::HashBuffer(Data, Length * sizeof(FParseCharType)).Hash);
		}

		/** Comparator */
		bool operator==(const FSmallStringView& RHS) const {
			return
				XXHash32 == RHS.XXHash32 &&
				Length == RHS.Length &&
				!FMemory::Memcmp(Begin, RHS.Begin, sizeof(FParseCharType) * Length);
		}

		// DW0-2
		FParseCharType* Begin;
		// DW2
		uint32_t XXHash32;
		uint16_t Length;
	};
	
	struct FConstSmallStringView
	{
		FConstSmallStringView() = default;
		
		FConstSmallStringView(const FSmallStringView& Mutable) : Begin(Mutable.Begin), XXHash32(Mutable.XXHash32), Length(Mutable.Length)
		{
			
		}
		
		/** Create from a text region */
		static FConstSmallStringView Text(const FParseCharType* Data)
		{
			return Text(Data, FCString::Strlen(Data));
		}
		
		/** Create from a text region */
		static FConstSmallStringView Text(const FParseCharType* Data, uint16 Length)
		{
			FConstSmallStringView Out;
			Out.Begin = Data;
			Out.XXHash32 = FSmallStringView::Hash(Data, Length);
			Out.Length = Length;
			return Out;
		}

		/** Comparator */
		bool operator==(const FConstSmallStringView& RHS) const {
			return
				XXHash32 == RHS.XXHash32 &&
				Length == RHS.Length &&
				!FMemory::Memcmp(Begin, RHS.Begin, sizeof(FParseCharType) * Length);
		}
		
		// DW0-2
		const FParseCharType* Begin;
		// DW2
		uint32_t XXHash32;
		uint16_t Length;
	};

	template<typename T>
	struct TSmallStringKeyFuncs : DefaultKeyFuncs<T>
	{
		static FORCEINLINE T GetSetKey(T K)
		{
			return K;
		}

		template <typename U>
		static FORCEINLINE T GetSetKey(const TPair<T, U>& P)
		{
			return P.Key;
		}
		
		static FORCEINLINE bool Matches(T A, T B)
		{
			return A == B;
		}
		
		static FORCEINLINE uint32 GetKeyHash(T Key)
		{
			return Key.XXHash32;
		}
	};

	using FSmallStringKeyFuncs      = TSmallStringKeyFuncs<FSmallStringView>;
	using FConstSmallStringKeyFuncs = TSmallStringKeyFuncs<FConstSmallStringView>;

	static_assert(sizeof(FToken) == 16,                "Unexpected FToken packing");
	static_assert(sizeof(FSmallStringView) == 16,      "Unexpected FSmallStringView packing");
	static_assert(sizeof(FConstSmallStringView) == 16, "Unexpected FConstSmallStringView packing");
}
