// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/ChunkedArray.h"
#include "SDCEToken.h"
#include <cctype>

namespace UE::ShaderMinifier::SDCE
{
	/** We're making heavy assumptions on the ansi bit-width */
	static_assert(std::is_same_v<FLookupCharType, char>, "SDCE-Tokenizer assumes ansi characters for lookups");

	/** Array size needed for a bit-width */
	template<typename T>
	constexpr uint32 TArraySizeFor = 1u << (sizeof(T) * 8);

	/** Mask size needed for a bit-width */
	template<typename T>
	constexpr uint32 TMask32SizeFor = TArraySizeFor<T> / 32;

	/** Invalid operation head */
	static constexpr int32_t TokenizerInvalidOpHead = INT32_MAX;

	struct FTokenizerSegmentAnsiNode
	{
		/** Token at this ansi node */
		ETokenType Type = ETokenType::Unknown;

		/** Child nodes */
		FTokenizerSegmentAnsiNode* Trees[TArraySizeFor<FLookupCharType>]{};
	};
	
	struct FTokenizerStatic
	{
		FTokenizerStatic()
		{
			// Populate all segments
#define X(N, S) AddSegment(ETokenType::N, S);
			UE_MINIFIER_SDCE_KEYWORD_TABLE
#undef X

			// Numeric 
			for (char C = '0'; C <= '9'; ++C)
			{
				AddCharBit(AlphaNumericMask, C);
			}

			// Lowercase alpha
			for (char C = 'a'; C <= 'z'; ++C)
			{
				AddCharBit(AlphaNumericMask, C);
				AddCharBit(AlphaMask, C);
			}

			// Uppercase alpha
			for (char C = 'A'; C <= 'Z'; ++C)
			{
				AddCharBit(AlphaNumericMask, C);
				AddCharBit(AlphaMask, C);
			}

			// All alpha's accept _
			AddCharBit(AlphaNumericMask, '_');
			AddCharBit(AlphaMask, '_');

			// All white-space like characters
			AddCharBit(SpaceMask, ' ');
			AddCharBit(SpaceMask, '\f');
			AddCharBit(SpaceMask, '\r');
			AddCharBit(SpaceMask, '\n');
			AddCharBit(SpaceMask, '\t');
			AddCharBit(SpaceMask, '\v');
		}

		void AddSegment(ETokenType Type, const FLookupViewType& Value)
		{
			FTokenizerSegmentAnsiNode* Base = &SegmentRoot;

			// Populate the segment node tree for the full length
			for (int32 i = 0; i < Value.Len(); i++)
			{
				FLookupCharType Char = Value[i];

				uint8 Index = static_cast<uint8>(Char);
				if (!Base->Trees[Index])
				{
					Base->Trees[Index] = &AnsiNodeAllocator[AnsiNodeAllocator.Emplace()];
				}

				Base = Base->Trees[Index];
			}

			Base->Type = Type;
		}

		static void AddCharBit(uint32_t Set[], FLookupCharType Value)
		{
			uint8 Index = static_cast<uint8>(Value);
			Set[Index / 32] |= 1u << (Index % 32);
		}

		static bool GetCharBit(uint32_t Set[], FLookupCharType Value)
		{
			uint8 Index = static_cast<uint8>(Value);
			return Set[Index / 32] & (1u << (Index % 32));
		}

		static bool IsDigit(FLookupCharType Ch)
		{
			return Ch >= '0' && Ch <= '9';
		}

		bool IsAlphaNumeric(FLookupCharType Ch)
		{
			return GetCharBit(AlphaNumericMask, Ch);
		}

		bool IsAlpha(FLookupCharType Ch)
		{
			return GetCharBit(AlphaMask, Ch);
		}

		bool IsSpace(FLookupCharType Ch)
		{
			return GetCharBit(SpaceMask, Ch);
		}

		// Lookup masks
		uint32_t AlphaMask       [TMask32SizeFor<FLookupCharType>]{};
		uint32_t AlphaNumericMask[TMask32SizeFor<FLookupCharType>]{};
		uint32_t SpaceMask       [TMask32SizeFor<FLookupCharType>]{};

		// Base segment
		FTokenizerSegmentAnsiNode SegmentRoot;

	private:
		// Chunked node allocator for coherence
		TChunkedArray<FTokenizerSegmentAnsiNode, sizeof(FTokenizerSegmentAnsiNode) * 256> AnsiNodeAllocator;
	} static TokenizerStatic;

	struct FShallowTokenizer
	{
		FShallowTokenizer() = default;

		/** Move constructor */
		FShallowTokenizer(FShallowTokenizer&& rhs) : Token(rhs.Token), OpHead(rhs.OpHead), Begin(rhs.Begin), End(rhs.End)
		{
			// Invalidate RHS op-head
			rhs.OpHead = TokenizerInvalidOpHead;
		}

		~FShallowTokenizer()
		{
			// If there's a valid op head, this is a branch that hasn't been accepted/rejected
			check(OpHead == TokenizerInvalidOpHead);
		}
		
		FShallowTokenizer& operator=(FShallowTokenizer&& rhs)
		{
			checkf(OpHead == TokenizerInvalidOpHead, TEXT("Assignment to a pending tokenizer"));
			Token = rhs.Token;
			OpHead = rhs.OpHead;
			Begin = rhs.Begin;
			End = rhs.End;
			
			// Invalidate RHS op-head
			rhs.OpHead = TokenizerInvalidOpHead;
			return *this;
		}

		/** Branch this tokenizer */
		FShallowTokenizer Branch() const
		{
			FShallowTokenizer Branch;
			Branch.OpHead = OpHead;
			Branch.Token = Token;
			Branch.Begin = Begin;
			Branch.End = End;
			return Branch;
		}

		/** Accept another tokenizer onto this */
		void Accept(FShallowTokenizer& RHS)
		{
			// We keep our own op-head, but reset the RHS
			RHS.OpHead = TokenizerInvalidOpHead;

			// Inherit token cursor
			Token = RHS.Token;
			Begin = RHS.Begin;
		}

		/** Check if the current token type is <Type> */
		bool Is(ETokenType Type)
		{
			return Token.Type == Type;
		}

		/** Check if the current token type is <Type> and consume it */
		bool IsConsume(ETokenType Type)
		{
			if (Is(Type))
			{
				Next();
				return true;
			}

			return false;
		}

		/** Get the current token and proceed */
		FToken Next()
		{
			FToken CurrentToken = Token;
			SkipWhitespace();
			NextInternal();
			return CurrentToken;
		}

		/** Actual tokenization internals */
		void NextInternal()
		{
			FParseCharType* TokenStart = Begin;
			if (Begin >= End)
			{
				return;
			}

			// Resolve static lifetime once
			FTokenizerStatic& Static = TokenizerStatic;

			// Numeric?
			if (Static.IsDigit(*Begin) || (*Begin == '.' && Static.IsDigit(PeekInternal(Begin, 1))))
			{
				bool bHasPunctuation = *Begin == '.';
				if (bHasPunctuation)
				{
					Begin++;
				}

				// Parse through the number
				while (Begin < End && (Static.IsDigit(*Begin) || (!bHasPunctuation && *Begin == '.')))
				{
					Begin++;
				}

				// Handle specializations
				if (Begin < End)
				{
					if (*Begin == 'x')
					{
						Begin++;
						while (Begin < End && Static.IsAlphaNumeric(*Begin))
						{
							Begin++;
						}
					}
					else if (*Begin == 'b')
					{
						Begin++;
						while (Begin < End && Static.IsDigit(*Begin))
						{
							Begin++;
						}
					}
					else if (*Begin == 'e')
					{
						Begin++;

						if (*Begin == '+' || *Begin == '-')
						{
							Begin++;
						}
				
						while (Begin < End && Static.IsDigit(*Begin))
						{
							Begin++;
						}
					}
					else if (*Begin == '#')
					{
						Begin++;

						while (Begin < End && Static.IsAlpha(*Begin))
						{
							Begin++;
						}
					}
				}

				// Handle typing
				if (Begin < End)
				{
					switch (std::tolower(static_cast<uint8>(*Begin)))
					{
					default:
						break;
					case 'u':
					case 'f':
					case 'd':
						Begin++;
					}
				}

				Token.Type = ETokenType::Numeric;
				Token.Begin = TokenStart;
				Token.Length = Begin - TokenStart;
				Token.XXHash32 = FSmallStringView::Hash(TokenStart, Token.Length);
				return;
			}

			// Current segment tree node
			FTokenizerSegmentAnsiNode* Tree = &Static.SegmentRoot;

			// Is this a potential segment?
			if (Tree->Trees[static_cast<uint8>(*Begin)])
			{
				FParseCharType* Shadow = Begin;

				// Keep traversing until we're out of nodes
				while (Shadow < End && Tree->Trees[static_cast<uint8>(*Shadow)])
				{
					Tree = Tree->Trees[static_cast<uint8>(*Shadow)];
					Shadow++;
				}

				// If the last character was alnum, the next cannot be
				if (!Static.IsAlphaNumeric(*(Shadow - 1)) || (!Static.IsAlphaNumeric(*Shadow)))
				{
					if (Tree->Type != ETokenType::Unknown)
					{
						Token = {};
						Token.Begin = TokenStart;
						Token.Type = Tree->Type;
						Begin = Shadow;
						return;
					}
				}
			}

			// Not a segment, just a regular identifier?
			if (Static.IsAlpha(*Begin))
			{
				while (Begin < End && Static.IsAlphaNumeric(*Begin))
				{
					Begin++;
				}

				Token.Type = ETokenType::ID;
				Token.Begin = TokenStart;
				Token.Length = Begin - TokenStart;
				Token.XXHash32 = FSmallStringView::Hash(TokenStart, Token.Length);
				return;
			}

			// End of stream?
			if (Begin >= End)
			{
				Token.Begin = End - 1;
				Token.Type = ETokenType::Unknown;
				Token.SetIDNoHash(FParseViewType(End - 1, 1));
				return;
			}

			// Unknown token, report as single character
			Token.Begin = TokenStart;
			Token.Type = ETokenType::Unknown;
			Token.SetIDNoHash(FParseViewType(TokenStart, ++Begin - TokenStart));
		}

		/** Peek a character */
		FParseCharType PeekInternal(const FParseCharType* Cursor, uint32 Offset = 0)
		{
			if (Cursor + Offset >= End)
			{
				return 0;
			}

			return *(Cursor + Offset);
		}

		/** Skip until we meet a character */
		bool SkipUntil(FParseCharType Ch)
		{
			while (Begin < End && *Begin != Ch)
			{
				Begin++;
			}
		
			bool bIs = *Begin == Ch;
			Next();
			return bIs;
		}

		/** Skip until we meet a set of characters */
		bool SkipUntil(FParseConstViewType ChSet)
		{
			while (Begin < End && !IsOneOf(*Begin, ChSet))
			{
				Begin++;
			}

			bool bIs = IsOneOf(*Begin, ChSet);
			Next();
			return bIs;
		}

		/** Skip all whitespaces */
		void SkipWhitespace()
		{
			while (Begin < End)
			{
				if (TokenizerStatic.IsSpace(*Begin))
				{
					Begin++;
				}

				// We're cheating, also skip preprocessor directives here
				else if (*Begin == '#')
				{
					while (Begin < End && *Begin != '\n')
					{
						Begin++;
					}
				}

				// Not a whitespace
				else
				{
					break;
				}
			}
		}

		/** Is this end of stream? */
		bool IsEOS()
		{
			SkipWhitespace();
			return Begin >= End;
		}

		/** Check if a character is in a set */
		static bool IsOneOf(char Ch, FParseConstViewType ChSet)
		{
			for (char Set : ChSet)
			{
				if (Set == Ch)
				{
					return true;
				}
			}
		
			return false;
		}

		/** Current cursor offset */
		uint32_t Offset() const
		{
			return End - Begin;
		}

		/** Current token */
		FToken Token;

		/** Operation buffer head, used for parser branching logic */
		int32 OpHead = 0;

		/** Cursors */
		FParseCharType* Begin = nullptr;
		FParseCharType* End   = nullptr;
	};
}
