// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderCompilerCommon.h"
#include "ShaderParameterParser.h"
#include "Misc/Base64.h"
#include "Misc/Compression.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Modules/ModuleManager.h"
#include "HAL/FileManager.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "String/RemoveFrom.h"
#include "ShaderCompilerDefinitions.h"
#include "ShaderPreprocessor.h"
#include "ShaderPreprocessTypes.h"
#include "ShaderSymbolExport.h"
#include "ShaderMinifier.h"
#include "Algo/Sort.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, ShaderCompilerCommon);

int16 GetNumUniformBuffersUsed(const FShaderCompilerResourceTable& InSRT)
{
	auto CountLambda = [&](const TArray<uint32>& In)
					{
						int16 LastIndex = -1;
						for (int32 i = 0; i < In.Num(); ++i)
						{
							auto BufferIndex = FRHIResourceTableEntry::GetUniformBufferIndex(In[i]);
							if (BufferIndex != static_cast<uint16>(FRHIResourceTableEntry::GetEndOfStreamToken()) )
							{
								LastIndex = FMath::Max(LastIndex, (int16)BufferIndex);
							}
						}

						return LastIndex + 1;
					};
	int16 Num = CountLambda(InSRT.SamplerMap);
	Num = FMath::Max(Num, (int16)CountLambda(InSRT.ShaderResourceViewMap));
	Num = FMath::Max(Num, (int16)CountLambda(InSRT.TextureMap));
	Num = FMath::Max(Num, (int16)CountLambda(InSRT.UnorderedAccessViewMap));
	Num = FMath::Max(Num, (int16)CountLambda(InSRT.ResourceCollectionMap));
	return Num;
}


void BuildResourceTableTokenStream(const TArray<uint32>& InResourceMap, int32 MaxBoundResourceTable, TArray<uint32>& OutTokenStream, bool bGenerateEmptyTokenStreamIfNoResources)
{
	if (bGenerateEmptyTokenStreamIfNoResources)
	{
		if (InResourceMap.Num() == 0)
		{
			return;
		}
	}

	// First we sort the resource map.
	TArray<uint32> SortedResourceMap = InResourceMap;
	SortedResourceMap.Sort();

	// The token stream begins with a table that contains offsets per bound uniform buffer.
	// This offset provides the start of the token stream.
	OutTokenStream.AddZeroed(MaxBoundResourceTable+1);
	auto LastBufferIndex = FRHIResourceTableEntry::GetEndOfStreamToken();
	for (int32 i = 0; i < SortedResourceMap.Num(); ++i)
	{
		auto BufferIndex = FRHIResourceTableEntry::GetUniformBufferIndex(SortedResourceMap[i]);
		if (BufferIndex != LastBufferIndex)
		{
			// Store the offset for resources from this buffer.
			OutTokenStream[BufferIndex] = OutTokenStream.Num();
			LastBufferIndex = BufferIndex;
		}
		OutTokenStream.Add(SortedResourceMap[i]);
	}

	// Add a token to mark the end of the stream. Not needed if there are no bound resources.
	if (OutTokenStream.Num())
	{
		OutTokenStream.Add(FRHIResourceTableEntry::GetEndOfStreamToken());
	}
}

void UE::ShaderCompilerCommon::BuildShaderResourceTable(const FShaderCompilerResourceTable& GenericSRT, FShaderResourceTable& OutSRT, bool bGenerateEmptyTokenStreamIfNoResources)
{
	// Copy over the bits indicating which resource tables are active.
	OutSRT.ResourceTableBits = GenericSRT.ResourceTableBits;

	OutSRT.ResourceTableLayoutHashes = GenericSRT.ResourceTableLayoutHashes;

	// Now build our token streams.
	BuildResourceTableTokenStream(GenericSRT.TextureMap,             GenericSRT.MaxBoundResourceTable, OutSRT.TextureMap,             bGenerateEmptyTokenStreamIfNoResources);
	BuildResourceTableTokenStream(GenericSRT.ShaderResourceViewMap,  GenericSRT.MaxBoundResourceTable, OutSRT.ShaderResourceViewMap,  bGenerateEmptyTokenStreamIfNoResources);
	BuildResourceTableTokenStream(GenericSRT.SamplerMap,             GenericSRT.MaxBoundResourceTable, OutSRT.SamplerMap,             bGenerateEmptyTokenStreamIfNoResources);
	BuildResourceTableTokenStream(GenericSRT.UnorderedAccessViewMap, GenericSRT.MaxBoundResourceTable, OutSRT.UnorderedAccessViewMap, bGenerateEmptyTokenStreamIfNoResources);
	BuildResourceTableTokenStream(GenericSRT.ResourceCollectionMap,  GenericSRT.MaxBoundResourceTable, OutSRT.ResourceCollectionMap,  bGenerateEmptyTokenStreamIfNoResources);
}

static bool DoesUniformBufferNeedReflectedMembers(const TMap<FString, FUniformBufferEntry>& UniformBufferMap, FStringView UniformBufferName)
{
	FStringView AdjustedUniformBufferName(UE::ShaderCompilerCommon::RemoveConstantBufferPrefix(UniformBufferName));

	if (const FUniformBufferEntry* Entry = UniformBufferMap.FindByHash(GetTypeHash(AdjustedUniformBufferName), AdjustedUniformBufferName))
	{
		return EnumHasAnyFlags(Entry->Flags, ERHIUniformBufferFlags::NeedsReflectedMembers);
	}

	return false;
}

bool BuildResourceTableMapping(
	const FShaderResourceTableMap& ResourceTableMap,
	const TMap<FString, FUniformBufferEntry>& UniformBufferMap,
	TBitArray<>& UsedUniformBufferSlots,
	FShaderParameterMap& ParameterMap,
	FShaderCompilerResourceTable& OutSRT)
{
	check(OutSRT.ResourceTableBits == 0);
	check(OutSRT.ResourceTableLayoutHashes.Num() == 0);

	// Build resource table mapping
	int32 MaxBoundResourceTable = -1;

	// Go through ALL the members of ALL the UB resources
	for (const FUniformResourceEntry& Entry : ResourceTableMap.Resources)
	{
		const FString& Name = Entry.UniformBufferMemberName;

		// If the shaders uses this member (eg View_PerlinNoise3DTexture)...
		if (TOptional<FParameterAllocation> Allocation = ParameterMap.FindAndRemoveParameterAllocation(Name))
		{
			FStringView UniformBufferName = Entry.GetUniformBufferName();

			const EShaderParameterType ParameterType = Allocation->Type;
			const bool bBindlessParameter = IsParameterBindless(ParameterType);

			// Force bindless "indices" to zero since they're not needed in SetResourcesFromTables
			const uint16 BaseIndex = bBindlessParameter ? 0 : Allocation->BaseIndex;

			if (DoesUniformBufferNeedReflectedMembers(UniformBufferMap, UniformBufferName))
			{
				FString RenamedMember = Name.Replace(TEXT("_"), TEXT("."));
				ParameterMap.AddParameterAllocation(*RenamedMember, Allocation->BufferIndex, Allocation->BaseIndex, Allocation->Size, Allocation->Type);

				// Force the parameter to be marked as bound
				ParameterMap.FindParameterAllocation(RenamedMember);
			}

			uint16 UniformBufferIndex = INDEX_NONE;

			// Add the UB itself as a parameter if not there
			if (TOptional<FParameterAllocation> UniformBufferParameter = ParameterMap.FindParameterAllocation(UniformBufferName))
			{
				UniformBufferIndex = UniformBufferParameter->BufferIndex;
			}
			else
			{
				UniformBufferIndex = UsedUniformBufferSlots.FindAndSetFirstZeroBit();
				ParameterMap.AddParameterAllocation(UniformBufferName, UniformBufferIndex, 0, 0, EShaderParameterType::UniformBuffer);
			}

			// Mark used UB index
			if (UniformBufferIndex >= sizeof(OutSRT.ResourceTableBits) * 8)
			{
				return false;
			}
			OutSRT.ResourceTableBits |= (1 << UniformBufferIndex);

			// How many resource tables max we'll use, and fill it with zeroes
			MaxBoundResourceTable = FMath::Max<int32>(MaxBoundResourceTable, (int32)UniformBufferIndex);

			auto ResourceMap = FRHIResourceTableEntry::Create(UniformBufferIndex, Entry.ResourceIndex, BaseIndex);
			switch( Entry.Type )
			{
			case UBMT_TEXTURE:
			case UBMT_RDG_TEXTURE:
				OutSRT.TextureMap.Add(ResourceMap);
				break;
			case UBMT_SAMPLER:
				OutSRT.SamplerMap.Add(ResourceMap);
				break;
			case UBMT_SRV:
			case UBMT_RDG_TEXTURE_SRV:
			case UBMT_RDG_TEXTURE_NON_PIXEL_SRV:
			case UBMT_RDG_BUFFER_SRV:
				OutSRT.ShaderResourceViewMap.Add(ResourceMap);
				break;
			case UBMT_RESOURCE_COLLECTION:
				OutSRT.ResourceCollectionMap.Add(ResourceMap);
				break;
			case UBMT_UAV:
			case UBMT_RDG_TEXTURE_UAV:
			case UBMT_RDG_BUFFER_UAV:
				OutSRT.UnorderedAccessViewMap.Add(ResourceMap);
				break;
			default:
				return false;
			}
		}
	}

	// Emit hashes for all uniform buffers in the parameter map. We need to include the ones without resources as well
	// (i.e. just constants), since the global uniform buffer bindings rely on valid hashes.
	for (const TPair<FString, FParameterAllocation>& KeyValue : ParameterMap.GetParameterMap())
	{
		const FString& UniformBufferName = KeyValue.Key;
		const FParameterAllocation& UniformBufferParameter = KeyValue.Value;

		if (UniformBufferParameter.Type == EShaderParameterType::UniformBuffer)
		{
			if (OutSRT.ResourceTableLayoutHashes.Num() <= UniformBufferParameter.BufferIndex)
			{
				OutSRT.ResourceTableLayoutHashes.SetNumZeroed(UniformBufferParameter.BufferIndex + 1);
			}

			// Data-driven uniform buffers will not have registered this information.
			if (const FUniformBufferEntry* UniformBufferEntry = UniformBufferMap.Find(UniformBufferName))
			{
				OutSRT.ResourceTableLayoutHashes[UniformBufferParameter.BufferIndex] = UniformBufferEntry->LayoutHash;
			}
		}
	}

	OutSRT.MaxBoundResourceTable = MaxBoundResourceTable;
	return true;
}

void CullGlobalUniformBuffers(const TMap<FString, FUniformBufferEntry>& UniformBufferMap, FShaderParameterMap& ParameterMap)
{
	TArray<FString> ParameterNames;
	ParameterMap.GetAllParameterNames(ParameterNames);

	for (const FString& Name : ParameterNames)
	{
		if (const FUniformBufferEntry* UniformBufferEntry = UniformBufferMap.Find(*Name))
		{
			// A uniform buffer that is bound per-shader keeps its allocation in the map.
			if (EnumHasAnyFlags(UniformBufferEntry->BindingFlags, EUniformBufferBindingFlags::Shader))
			{
				continue;
			}

			ParameterMap.RemoveParameterAllocation(Name);
		}
	}
}

template <typename CharType>
static bool IsSpaceOrTabOrEOL(CharType Char)
{
	return Char == ' ' || Char == '\t' || Char == '\n' || Char == '\r';
}

template <typename StrCharType, typename SearchCharType>
static const StrCharType* FindNextChar(const StrCharType* ReadStart, SearchCharType SearchChar)
{
	const StrCharType* SearchPtr = ReadStart;
	while (*SearchPtr && *SearchPtr != SearchChar)
	{
		SearchPtr++;
	}
	return SearchPtr;
}

template <typename CharType>
const CharType* FindNextWhitespace(const CharType* StringPtr)
{
	while (*StringPtr && !IsSpaceOrTabOrEOL(*StringPtr))
	{
		StringPtr++;
	}

	if (*StringPtr && IsSpaceOrTabOrEOL(*StringPtr))
	{
		return StringPtr;
	}
	else
	{
		return nullptr;
	}
}

template <typename CharType>
const CharType* FindNextNonWhitespace(const CharType* StringPtr)
{
	while (*StringPtr && IsSpaceOrTabOrEOL(*StringPtr))
	{
		StringPtr++;
	}

	if (*StringPtr && !IsSpaceOrTabOrEOL(*StringPtr))
	{
		return StringPtr;
	}

	return nullptr;
}

template <typename CharType>
const CharType* FindPreviousNonWhitespace(const CharType* StringPtr)
{
	do
	{
		StringPtr--;
	} while (*StringPtr && IsSpaceOrTabOrEOL(*StringPtr));

	if (*StringPtr && !IsSpaceOrTabOrEOL(*StringPtr))
	{
		return StringPtr;
	}

	return nullptr;
}

template <typename CharType>
const CharType* FindMatchingClosingParenthesis(const CharType* OpeningCharPtr)	{ return FindMatchingBlock<CharType>(OpeningCharPtr, '(', ')'); };

// See MSDN HLSL 'Symbol Name Restrictions' doc
template <typename CharType>
inline bool IsValidHLSLIdentifierCharacter(CharType Char)
{
	return (Char >= 'a' && Char <= 'z') ||
		(Char >= 'A' && Char <= 'Z') ||
		(Char >= '0' && Char <= '9') ||
		Char == '_';
}

void ParseHLSLTypeName(const TCHAR* SearchString, const TCHAR*& TypeNameStartPtr, const TCHAR*& TypeNameEndPtr)
{
	TypeNameStartPtr = FindNextNonWhitespace(SearchString);
	check(TypeNameStartPtr);

	TypeNameEndPtr = TypeNameStartPtr;
	int32 Depth = 0;

	const TCHAR* NextWhitespace = FindNextWhitespace(TypeNameStartPtr);
	const TCHAR* PotentialExtraTypeInfoPtr = NextWhitespace ? FindNextNonWhitespace(NextWhitespace) : nullptr;

	// Find terminating whitespace, but skip over trailing ' < float4 >'
	while (*TypeNameEndPtr)
	{
		if (*TypeNameEndPtr == '<')
		{
			Depth++;
		}
		else if (*TypeNameEndPtr == '>')
		{
			Depth--;
		}
		else if (Depth == 0 
			&& IsSpaceOrTabOrEOL(*TypeNameEndPtr)
			// If we found a '<', we must not accept any whitespace before it
			&& (!PotentialExtraTypeInfoPtr || *PotentialExtraTypeInfoPtr != '<' || TypeNameEndPtr > PotentialExtraTypeInfoPtr))
		{
			break;
		}

		TypeNameEndPtr++;
	}

	check(TypeNameEndPtr);
}

template<typename CharType, typename ViewType>
ViewType ParseHLSLSymbolName(const CharType* SearchString)
{
	const CharType* SymbolNameStartPtr = FindNextNonWhitespace(SearchString);
	check(SymbolNameStartPtr);

	const CharType* SymbolNameEndPtr = SymbolNameStartPtr;
	while (*SymbolNameEndPtr && IsValidHLSLIdentifierCharacter(*SymbolNameEndPtr))
	{
		SymbolNameEndPtr++;
	}

	return ViewType(SymbolNameStartPtr, SymbolNameEndPtr - SymbolNameStartPtr);
}

const TCHAR* ParseHLSLSymbolName(const TCHAR* SearchString, FString& SymbolName)
{
	FStringView Result = ParseHLSLSymbolName<TCHAR, FStringView>(SearchString);

	SymbolName = FString(Result);

	return Result.GetData() + Result.Len();
}

FStringView FindNextHLSLDefinitionOfType(FStringView Typename, FStringView StartPos)
{
	// handle both the case where identifier for declaration immediately precedes a ; and has whitespace separating the two
	const TCHAR* NextWhitespace;
	const TCHAR* NextNonWhitespace;
	FStringView SymbolName;

	NextWhitespace = FindNextWhitespace(StartPos.GetData());
	if (NextWhitespace == StartPos.GetData())
	{
		NextNonWhitespace = FindNextNonWhitespace(NextWhitespace);
		SymbolName = ParseHLSLSymbolName<TCHAR, FStringView>(NextNonWhitespace);	
		NextNonWhitespace = FindNextNonWhitespace(NextNonWhitespace + SymbolName.Len());
		if (NextNonWhitespace && (*NextNonWhitespace == ';'))
		{
			return SymbolName;
		}
	}
	return {};
}

FStringView UE::ShaderCompilerCommon::RemoveConstantBufferPrefix(FStringView InName)
{
	return UE::String::RemoveFromStart(InName, FStringView(UE::ShaderCompilerCommon::kUniformBufferConstantBufferPrefix));
}

FString UE::ShaderCompilerCommon::RemoveConstantBufferPrefix(const FString& InName)
{
	return FString(RemoveConstantBufferPrefix(FStringView(InName)));
}

bool UE::ShaderCompilerCommon::ValidatePackedResourceCounts(FShaderCompilerOutput& Output, const FShaderCodePackedResourceCounts& PackedResourceCounts)
{
	if (Output.bSucceeded)
	{
		auto GetAllResourcesOfType = [&](EShaderParameterType InType)
		{
			const TArray<FStringView> AllNames = Output.ParameterMap.GetAllParameterNamesOfType(InType);
			if (AllNames.IsEmpty())
			{
				return FString();
			}
			return FString::Join(AllNames, TEXT(", "));
		};

		if (EnumHasAnyFlags(PackedResourceCounts.UsageFlags, EShaderResourceUsageFlags::BindlessResources) && PackedResourceCounts.NumSRVs > 0)
		{
			const FString Names = GetAllResourcesOfType(EShaderParameterType::SRV);
			Output.AddErrorf(TEXT("Shader is mixing bindless resources with non-bindless resources. %d SRV slots were detected: %s"), PackedResourceCounts.NumSRVs, *Names);
			Output.bSucceeded = false;
		}

		if (EnumHasAnyFlags(PackedResourceCounts.UsageFlags, EShaderResourceUsageFlags::BindlessResources) && PackedResourceCounts.NumUAVs > 0)
		{
			const FString Names = GetAllResourcesOfType(EShaderParameterType::UAV);
			Output.AddErrorf(TEXT("Shader is mixing bindless resources with non-bindless resources. %d UAV slots were detected: %s"), PackedResourceCounts.NumUAVs, *Names);
			Output.bSucceeded = false;
		}

		if (EnumHasAnyFlags(PackedResourceCounts.UsageFlags, EShaderResourceUsageFlags::BindlessSamplers) && PackedResourceCounts.NumSamplers > 0)
		{
			const FString Names = GetAllResourcesOfType(EShaderParameterType::Sampler);
			Output.AddErrorf(TEXT("Shader is mixing bindless samplers with non-bindless samplers. %d sampler slots were detected: %s"), PackedResourceCounts.NumSamplers, *Names);
			Output.bSucceeded = false;
		}
	}

	return Output.bSucceeded;
}

void UE::ShaderCompilerCommon::ParseRayTracingEntryPoint(const FStringView& Input, FStringView& OutMain, FStringView& OutAnyHit, FStringView& OutIntersection)
{
	auto ParseEntry = [&Input](const FStringView& Marker)
	{
		FStringView Result;
		const int32 BeginIndex = UE::String::FindFirst(Input, Marker, ESearchCase::IgnoreCase);
		if (BeginIndex != INDEX_NONE)
		{
			int32 EndIndex = UE::String::FindFirst(Input.Mid(BeginIndex), TEXTVIEW(" "), ESearchCase::IgnoreCase);
			if (EndIndex == INDEX_NONE)
			{
				EndIndex = Input.Len() + 1;
			}
			else
			{
				EndIndex += BeginIndex;
			}
			const int32 MarkerLen = Marker.Len();
			const int32 Count = EndIndex - BeginIndex;
			Result = Input.Mid(BeginIndex + MarkerLen, Count - MarkerLen);
		}
		return Result;
	};

	OutMain = ParseEntry(TEXTVIEW("closesthit="));
	OutAnyHit = ParseEntry(TEXTVIEW("anyhit="));
	OutIntersection = ParseEntry(TEXTVIEW("intersection="));

	// If complex hit group entry is not specified, assume a single verbatim entry point
	if (OutMain.IsEmpty() && OutAnyHit.IsEmpty() && OutIntersection.IsEmpty())
	{
		OutMain = Input;
	}
}

void UE::ShaderCompilerCommon::ParseRayTracingEntryPoint(const FString& Input, FString& OutMain, FString& OutAnyHit, FString& OutIntersection)
{
	FStringView OutMainView;
	FStringView OutAnyHitView;
	FStringView OutIntersectionView;
	ParseRayTracingEntryPoint(Input, OutMainView, OutAnyHitView, OutIntersectionView);

	OutMain = OutMainView;
	OutAnyHit = OutAnyHitView;
	OutIntersection = OutIntersectionView;
}


bool UE::ShaderCompilerCommon::RemoveDeadCode(FShaderSource& InOutPreprocessedShaderSource, TConstArrayView<FStringView> InRequiredSymbols, TArray<FShaderCompilerError>& OutErrors)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(RemoveDeadCode);

	UE::ShaderMinifier::EMinifyShaderFlags ExtraFlags = UE::ShaderMinifier::EMinifyShaderFlags::None;

#if 0 // Extra features that may be useful during development / debugging
	ExtraFlags |= UE::ShaderMinifier::EMinifyShaderFlags::OutputReasons // Output a comment every struct/function describing why it was included (i.e. which code block uses it)
	           |  UE::ShaderMinifier::EMinifyShaderFlags::OutputStats;  // Output a comment detailing how many blocks of each type (functions/structs/etc.) were emitted
#endif

	TArray<FShaderSource::FStringType> ConvertedRequiredSymbols;
	TArray<FShaderSource::FViewType> RequiredSymbolViews;
	for (FStringView InSymbol : InRequiredSymbols)
	{
		FShaderSource::FStringType& ConvertedString = ConvertedRequiredSymbols.AddDefaulted_GetRef();
		ConvertedString.Append(InSymbol);
		RequiredSymbolViews.Add(FShaderSource::FViewType(ConvertedString));
	}

	UE::ShaderMinifier::FMinifiedShader Minified  = UE::ShaderMinifier::Minify(InOutPreprocessedShaderSource, RequiredSymbolViews,
		  UE::ShaderMinifier::EMinifyShaderFlags::OutputCommentLines // Preserve comments that were left after preprocessing
		| UE::ShaderMinifier::EMinifyShaderFlags::OutputLines        // Emit #line directives
		| ExtraFlags);

	if (Minified.Success())
	{
		InOutPreprocessedShaderSource = MoveTemp(Minified.Code);
		return true;
	}
	else
	{
		OutErrors.Add(TEXT("warning: Shader minification failed."));
		return false;
	}
}

bool UE::ShaderCompilerCommon::RemoveDeadCode(FShaderSource& InOutPreprocessedShaderSource, const FString& EntryPoint, TArray<FShaderCompilerError>& OutErrors)
{
	return UE::ShaderCompilerCommon::RemoveDeadCode(InOutPreprocessedShaderSource, EntryPoint, {}, OutErrors);
}

bool UE::ShaderCompilerCommon::RemoveDeadCode(FShaderSource& InOutPreprocessedShaderSource, const FString& EntryPoint, TConstArrayView<FStringView> InRequiredSymbols, TArray<FShaderCompilerError>& OutErrors)
{
	TArray<FStringView> RequiredSymbols;

	FStringView EntryMain;
	FStringView EntryAnyHit;
	FStringView EntryIntersection;
	UE::ShaderCompilerCommon::ParseRayTracingEntryPoint(EntryPoint, EntryMain, EntryAnyHit, EntryIntersection);

	RequiredSymbols.Add(EntryMain);

	if (!EntryAnyHit.IsEmpty())
	{
		RequiredSymbols.Add(EntryAnyHit);
	}

	if (!EntryIntersection.IsEmpty())
	{
		RequiredSymbols.Add(EntryIntersection);
	}

	for (FStringView Symbol : InRequiredSymbols)
	{
		RequiredSymbols.Add(Symbol);
	}

	return UE::ShaderCompilerCommon::RemoveDeadCode(InOutPreprocessedShaderSource, RequiredSymbols, OutErrors);
}

inline bool IsDescriptorRangeParameter(const FShaderParameterParser& ShaderParameterParser, FStringView MemberName)
{
	if (const FShaderParameterParser::FParsedShaderParameter* ParsedParameter = ShaderParameterParser.FindParameterInfosUnsafe(MemberName))
	{
		return (ParsedParameter->BaseType == UBMT_DESCRIPTOR_RANGE);
	}
	return false;
}

void HandleReflectedGlobalConstantBufferMember(
	const FShaderCompilerInput& Input,
	const FShaderParameterParser& ShaderParameterParser,
	FStringView InMemberName,
	uint32 ConstantBufferIndex,
	int32 ReflectionOffset,
	int32 ReflectionSize,
	FShaderCompilerOutput& Output
)
{
	FStringView MemberName = InMemberName;
	EShaderParameterType ParameterType = FShaderParameterParser::ParseAndRemoveBindlessParameterPrefix(MemberName);

	if (ParameterType == EShaderParameterType::LooseData && IsDescriptorRangeParameter(ShaderParameterParser, MemberName))
	{
		ParameterType = EShaderParameterType::DescriptorRange;
	}

	Output.ParameterMap.AddParameterAllocation(
		MemberName,
		ConstantBufferIndex,
		ReflectionOffset,
		ReflectionSize,
		ParameterType
	);
}

void HandleReflectedGlobalConstantBufferMember(
	FStringView InMemberName,
	uint32 ConstantBufferIndex,
	int32 ReflectionOffset,
	int32 ReflectionSize,
	FShaderCompilerOutput& Output
)
{
	FStringView MemberName = InMemberName;
	const EShaderParameterType ParameterType = FShaderParameterParser::ParseAndRemoveBindlessParameterPrefix(MemberName);

	Output.ParameterMap.AddParameterAllocation(
		MemberName,
		ConstantBufferIndex,
		ReflectionOffset,
		ReflectionSize,
		ParameterType
	);
}

void HandleReflectedUniformBufferConstantBufferMember(
	const FShaderCompilerInput& Input,
	EUniformBufferMemberReflectionReason Reason,
	FStringView UniformBufferName,
	int32 UniformBufferSlot,
	FStringView InMemberName,
	int32 ReflectionOffset,
	int32 ReflectionSize,
	FShaderCompilerOutput& Output
)
{
	FStringView MemberName = InMemberName;
	const EShaderParameterType ParameterType = FShaderParameterParser::ParseAndRemoveBindlessParameterPrefix(MemberName);

	bool bAdd = EnumHasAnyFlags(Reason, EUniformBufferMemberReflectionReason::NeedsReflection);
	if (EnumHasAnyFlags(Reason, EUniformBufferMemberReflectionReason::Bindless))
	{
		bAdd |= (ParameterType != EShaderParameterType::LooseData);
	}

	if (EnumHasAnyFlags(Reason, EUniformBufferMemberReflectionReason::MaterialUBBindError))
	{
#define SVARG(N) N.Len(), N.GetData()
		Output.AddErrorf(TEXT("%s: Unauthorized to use %.*s from %.*s at frequency %s"),
			*Input.GenerateShaderName(),
			SVARG(InMemberName),
			SVARG(UniformBufferName),
			GetShaderFrequencyString(Output.Target.GetFrequency()));
#undef SVARG
		Output.bSucceeded = false;
	}

	if (bAdd)
	{
		Output.ParameterMap.AddParameterAllocation(
			MemberName,
			UniformBufferSlot,
			ReflectionOffset,
			1,
			ParameterType
		);
	}
}

void HandleReflectedRootConstantBufferMember(
	const FShaderCompilerInput& Input,
	const FShaderParameterParser& ShaderParameterParser,
	FStringView InMemberName,
	int32 ReflectionOffset,
	int32 ReflectionSize,
	FShaderCompilerOutput& Output
)
{
	ShaderParameterParser.ValidateShaderParameterType(Input, InMemberName, ReflectionOffset, ReflectionSize, Output);

	FStringView MemberName = InMemberName;
	EShaderParameterType ParameterType = FShaderParameterParser::ParseAndRemoveBindlessParameterPrefix(MemberName);

	if (ParameterType == EShaderParameterType::LooseData && IsDescriptorRangeParameter(ShaderParameterParser, MemberName))
	{
		ParameterType = EShaderParameterType::DescriptorRange;
	}

	// Root constant buffer members only need reflection when they are specific to resources
	if (ParameterType != EShaderParameterType::LooseData)
	{
		Output.ParameterMap.AddParameterAllocation(
			MemberName,
			FShaderParametersMetadata::kRootCBufferBindingIndex,
			ReflectionOffset,
			1,
			ParameterType
		);
	}
}

void HandleReflectedRootConstantBuffer(
	int32 ConstantBufferSize,
	FShaderCompilerOutput& CompilerOutput
)
{
	CompilerOutput.ParameterMap.AddParameterAllocation(
		FShaderParametersMetadata::kRootUniformBufferBindingName,
		FShaderParametersMetadata::kRootCBufferBindingIndex,
		0,
		static_cast<uint16>(ConstantBufferSize),
		EShaderParameterType::LooseData);
}

bool VerifyBoundUniform(const FShaderCompilerInput& Input, FStringView UniformBufferName)
{
	EShaderFrequency ShaderFrequency = Input.Target.GetFrequency();
	bool bUsesMaterialVSUB = (ShaderFrequency == SF_Vertex || ShaderFrequency == SF_Mesh);
	if (!bUsesMaterialVSUB)
	{
		bUsesMaterialVSUB = Input.Environment.GetCompileArgument(TEXT("SHADER_USES_ONLY_VERTEX_DATA"), false) && Input.Environment.GetCompileArgument(TEXT("SHADER_SUPPORTS_MATERIAL_VS_UB"), false);
	}

	const TArray<FString>& DisallowedUniformBufferNames = (bUsesMaterialVSUB) ? Input.VSDisallowedUniformBufferNames : Input.NonVSDisallowedUniformBufferNames;
	if (DisallowedUniformBufferNames.Contains(UniformBufferName))
	{
		return false;
	}

	return true;
}

void HandleReflectedUniformBuffer(
	const FShaderCompilerInput& Input,
	FStringView UniformBufferName,
	int32 ReflectionSlot,
	int32 BaseIndex,
	int32 BufferSize,
	FShaderCompilerOutput& CompilerOutput
)
{
	FStringView AdjustedUniformBufferName(UE::ShaderCompilerCommon::RemoveConstantBufferPrefix(UniformBufferName));

	CompilerOutput.ParameterMap.AddParameterAllocation(
		AdjustedUniformBufferName,
		ReflectionSlot,
		BaseIndex,
		BufferSize,
		EShaderParameterType::UniformBuffer
	);

	if (!VerifyBoundUniform(Input, AdjustedUniformBufferName))
	{
#define SVARG(N) N.Len(), N.GetData()
		FString ErrorMessage = FString::Printf(TEXT("%s: Unauthorized to use from %.*s at frequency %s"), 
			*Input.GenerateShaderName(),
			SVARG(AdjustedUniformBufferName),
			GetShaderFrequencyString(CompilerOutput.Target.GetFrequency()));
#undef SVARG

		FShaderCompilerError Error(MoveTemp(ErrorMessage));
		CompilerOutput.Errors.Add(Error);
		CompilerOutput.bSucceeded = false;
	}

}

EUniformBufferMemberReflectionReason ShouldReflectUniformBufferMembers(const FShaderCompilerInput& Input, FStringView UniformBufferName)
{
	EUniformBufferMemberReflectionReason Reason{};

	if (Input.IsBindlessEnabled())
	{
		Reason |= EUniformBufferMemberReflectionReason::Bindless;
	}

	if (DoesUniformBufferNeedReflectedMembers(Input.Environment.UniformBufferMap, UniformBufferName))
	{
		Reason |= EUniformBufferMemberReflectionReason::NeedsReflection;
	}

	if (!VerifyBoundUniform(Input, UniformBufferName))
	{
		Reason |= EUniformBufferMemberReflectionReason::MaterialUBBindError;
	}

	return Reason;
}

void HandleReflectedShaderResource(
	FStringView ResourceName,
	int32 BindOffset,
	int32 ReflectionSlot,
	int32 BindCount,
	FShaderCompilerOutput& CompilerOutput
)
{
	CompilerOutput.ParameterMap.AddParameterAllocation(
		ResourceName,
		BindOffset,
		ReflectionSlot,
		BindCount,
		EShaderParameterType::SRV
	);
}

void UpdateStructuredBufferStride(
	const FShaderCompilerInput& Input,
	FStringView ResourceName,
	uint16 BindPoint,
	uint16 Stride,
	FShaderCompilerOutput& CompilerOutput
)
{
	if (BindPoint <= UINT16_MAX && Stride <= UINT16_MAX)
	{
		CompilerOutput.ParametersStrideToValidate.Add(FShaderCodeValidationStride{ BindPoint, Stride });
	}
	else
	{
		FString ErrorMessage =
			FString::Printf(TEXT("%s: Failed to set stride on parameter %.*s: Bind point %d, Stride %d"),
				*Input.GenerateShaderName(),
				ResourceName.Len(), ResourceName.GetData(),
				BindPoint,
				Stride
			);
		CompilerOutput.Errors.Add(FShaderCompilerError(*ErrorMessage));
	}
}

void AddShaderValidationSRVType(uint16 BindPoint,
							EShaderCodeResourceBindingType TypeDecl,
							FShaderCompilerOutput& CompilerOutput)
{
	if (BindPoint <= UINT16_MAX)
	{
		CompilerOutput.ParametersSRVTypeToValidate.Add(FShaderCodeValidationType{ BindPoint, TypeDecl });
	}
}

void AddShaderValidationUAVType(uint16 BindPoint,
							EShaderCodeResourceBindingType TypeDecl,
							FShaderCompilerOutput& CompilerOutput)
{
	if (BindPoint <= UINT16_MAX)
	{
		CompilerOutput.ParametersUAVTypeToValidate.Add(FShaderCodeValidationType{ BindPoint, TypeDecl });
	}
}

void AddShaderValidationUBSize(uint16 BindPoint,
							uint32_t Size,
							FShaderCompilerOutput& CompilerOutput)
{
	if (BindPoint <= UINT16_MAX)
	{
		CompilerOutput.ParametersUBSizeToValidate.Add(FShaderCodeValidationUBSize{ BindPoint, Size });
	}
}
 
void HandleReflectedShaderUAV(
	FStringView UAVName,
	int32 BindOffset,
	int32 ReflectionSlot,
	int32 BindCount,
	FShaderCompilerOutput& CompilerOutput
)
{
	CompilerOutput.ParameterMap.AddParameterAllocation(
		UAVName,
		BindOffset,
		ReflectionSlot,
		BindCount,
		EShaderParameterType::UAV
	);
}

void HandleReflectedShaderSampler(
	FStringView SamplerName,
	int32 BindOffset,
	int32 ReflectionSlot,
	int32 BindCount,
	FShaderCompilerOutput& CompilerOutput
)
{
	CompilerOutput.ParameterMap.AddParameterAllocation(
		SamplerName,
		BindOffset,
		ReflectionSlot,
		BindCount,
		EShaderParameterType::Sampler
	);
}

void AddNoteToDisplayShaderParameterStructureOnCppSide(
	const FShaderParametersMetadata* ParametersStructure,
	FShaderCompilerOutput& CompilerOutput)
{
	FShaderCompilerError Error;
	Error.StrippedErrorMessage = FString::Printf(
		TEXT("Note: Definition of structure %s"),
		ParametersStructure->GetStructTypeName());
	Error.ErrorVirtualFilePath = ANSI_TO_TCHAR(ParametersStructure->GetFileName());
	Error.ErrorLineString = FString::FromInt(ParametersStructure->GetFileLine());

	CompilerOutput.Errors.Add(Error);
}

void AddUnboundShaderParameterError(
	const FShaderCompilerInput& CompilerInput,
	const FShaderParameterParser& ShaderParameterParser,
	FStringView ParameterBindingName,
	FShaderCompilerOutput& CompilerOutput)
{
	check(CompilerInput.RootParametersStructure);

	const FShaderParameterParser::FParsedShaderParameter& Member = ShaderParameterParser.FindParameterInfos(ParameterBindingName);
	check(!EnumHasAnyFlags(Member.Flags, EParsedShaderParameterFlags::WasInRootParameters));

	FShaderCompilerError Error(FString::Printf(
		TEXT("Error: Shader parameter %.*s could not be bound to %s's shader parameter structure %s."),
		ParameterBindingName.Len(), ParameterBindingName.GetData(),
		*CompilerInput.ShaderName,
		CompilerInput.RootParametersStructure->GetStructTypeName()));
	ShaderParameterParser.GetParameterFileAndLine(Member, Error.ErrorVirtualFilePath, Error.ErrorLineString);

	CompilerOutput.Errors.Add(Error);
	CompilerOutput.bSucceeded = false;

	AddNoteToDisplayShaderParameterStructureOnCppSide(CompilerInput.RootParametersStructure, CompilerOutput);
}

struct FUniformBufferMemberInfo
{
	// eg View.WorldToClip
	FString NameAsStructMember;
	// eg View_WorldToClip
	FString GlobalName;
};

struct FUniformBufferInfo
{
	int32 DefinitionEndOffset;
	TArray<FUniformBufferMemberInfo> Members;
};

struct FUniformBufferMemberInfoNew
{
	// eg View.WorldToClip
	FShaderSource::FViewType NameAsStructMember;
	// eg View_WorldToClip
	FShaderSource::FViewType GlobalName;

	bool operator<(const FUniformBufferMemberInfoNew& Other)
	{
		if (NameAsStructMember.Len() != Other.NameAsStructMember.Len())
		{
			return NameAsStructMember.Len() < Other.NameAsStructMember.Len();
		}
		else
		{
			return NameAsStructMember.Compare(Other.NameAsStructMember, ESearchCase::CaseSensitive) < 0;
		}
	}
};

// Index and count of subset of members
struct FUniformBufferMemberView
{
	int32 MemberOffset;
	int32 MemberCount;
};

struct FUniformBufferInfoNew
{
	FShaderSource::FViewType Name;
	int32 NextWithSameLength;							// Linked list of uniform buffer infos with same name length
	TArray<FUniformBufferMemberInfoNew> Members;		// Members sorted by length
	TArray<FUniformBufferMemberView> MembersByLength;	// Offset and count of members of a given length
};

// Tracks the offset and length of commented out uniform buffer declarations in the source code, so we can compact them out
struct FUniformBufferSpan
{
	int32 Offset;
	int32 Length;
};

// Compacts spaces out of a compound identifier.  Returns the new end pointer of the compacted identifier.
// End and result pointers are exclusive (length of the string is End - Start).
static FShaderSource::CharType* CompactCompoundIdentifier(FShaderSource::CharType* Start, FShaderSource::CharType* End)
{
	// Find first whitespace in the identifier, if present
	FShaderSource::CharType* ReadChar;
	for (ReadChar = Start; ReadChar < End; ++ReadChar)
	{
		if (IsSpaceOrTabOrEOL(*ReadChar))
		{
			break;
		}
	}
	if (ReadChar == End)
	{
		// No whitespace, we're done!
		return End;
	}

	// Found some whitespace, so we need to compact the non-whitespace, swapping the whitespace to the end of the range
	// WriteChar here will be the first whitespace character that we need to compact into.
	FShaderSource::CharType* WriteChar = ReadChar;
	for (++ReadChar; ReadChar < End; ++ReadChar)
	{
		// If the current read character is non-whitespace, compact it down
		if (!IsSpaceOrTabOrEOL(*ReadChar))
		{
			Swap(*ReadChar, *WriteChar);
			WriteChar++;
		}
	}
	return WriteChar;
}

const FShaderSource::CharType* ParseUniformBufferDefinition(const FShaderSource::CharType* ReadStart, TArray<FUniformBufferInfoNew>& UniformBufferInfos, uint64 UniformBufferFilter[64], int32 UniformBuffersByLength[64])
{
	// TODO:  should we check for an existing item?  In my testing, there's only one uniform buffer declaration with a given name,
	// but the original code used a map, theoretically allowing for multiple.
	int32 InfoIndex = UniformBufferInfos.AddDefaulted();
	FUniformBufferInfoNew& Info = UniformBufferInfos[InfoIndex];

	Info.Name = ParseHLSLSymbolName<FShaderSource::CharType, FShaderSource::FViewType>(ReadStart);
	check(Info.Name.Len() < 64);

	const FShaderSource::CharType* OpeningBrace = FindNextChar(ReadStart, '{');
	const FShaderSource::CharType* ClosingBrace = FindMatchingClosingBrace(OpeningBrace + 1);

	const FShaderSource::CharType* CurrentParseStart = OpeningBrace + 1;
	const FShaderSource::CharType* NextSemicolon = FindNextChar(CurrentParseStart, ';');

	while (NextSemicolon < ClosingBrace)
	{
		const FShaderSource::CharType* NextSeparator = FindNextChar(CurrentParseStart, '=');
		if (NextSeparator < NextSemicolon)
		{
			const FShaderSource::CharType* StructStart = CurrentParseStart;
			const FShaderSource::CharType* StructEnd = NextSeparator;

			const FShaderSource::CharType* GlobalStart = NextSeparator + 1;
			const FShaderSource::CharType* GlobalEnd = NextSemicolon;

			while (IsSpaceOrTabOrEOL(*StructStart))
			{
				StructStart++;
			}
			while (IsSpaceOrTabOrEOL(*GlobalStart))
			{
				GlobalStart++;
			}

			StructEnd = CompactCompoundIdentifier(const_cast<FShaderSource::CharType*>(StructStart), const_cast<FShaderSource::CharType*>(StructEnd));
			GlobalEnd = CompactCompoundIdentifier(const_cast<FShaderSource::CharType*>(GlobalStart), const_cast<FShaderSource::CharType*>(GlobalEnd));

			FShaderSource::FViewType StructName(StructStart, StructEnd - StructStart);
			FShaderSource::FViewType GlobalName(GlobalStart, GlobalEnd - GlobalStart);

			// Avoid unnecessary conversions
			if (StructName.Len() != GlobalName.Len() || FShaderSource::FCStringType::Strncmp(StructName.GetData(), GlobalName.GetData(), StructName.Len()) != 0)
			{
				FUniformBufferMemberInfoNew NewMemberInfo;
				NewMemberInfo.NameAsStructMember = StructName;
				NewMemberInfo.GlobalName = GlobalName;

				// Need to be able to replace strings in place, so make sure GlobalName will fit in space of NameAsStructMember
				check(NewMemberInfo.NameAsStructMember.Len() >= NewMemberInfo.GlobalName.Len());

				Info.Members.Add(NewMemberInfo);
			}
		}

		CurrentParseStart = NextSemicolon + 1;
		NextSemicolon = FindNextChar(CurrentParseStart, ';');
	}

	const FShaderSource::CharType* EndPtr = ClosingBrace;

	// Skip to the end of the UniformBuffer
	while (*EndPtr && *EndPtr != ';')
	{
		EndPtr++;
	}

	if (Info.Members.Num())
	{
		// We have members.  Sort them.  Note that the sort is by length first, not alphabetical, so the last item will be the longest.
		Algo::Sort(Info.Members);

		int32 MaxLen = Info.Members.Last().NameAsStructMember.Len();

		// Initialize table with offset of first member with a given length, and the count of members of that length (going backwards so the
		// index of the first element of a given size is the last one written to "MemberOffset").
		Info.MembersByLength.SetNumZeroed(MaxLen + 1);

		for (int32 MemberIndex = Info.Members.Num() - 1; MemberIndex >= 0; MemberIndex--)
		{
			int32 CurrentMemberLen = Info.Members[MemberIndex].NameAsStructMember.Len();
			Info.MembersByLength[CurrentMemberLen].MemberOffset = MemberIndex;
			Info.MembersByLength[CurrentMemberLen].MemberCount++;
		}

		// Initialize the uniform buffer name filter.  The filter is a mask based on the first character of the name (minus 64 so valid token
		// starting characters which are in ASCII range 64..127 fit in 64 bits).  We can quickly check if a token of the given length and start
		// character might be one we care about.
		UniformBufferFilter[Info.Name.Len()] |= 1ull << (Info.Name[0] - 64);

		// Add to linked list of uniform buffers by name length
		Info.NextWithSameLength = UniformBuffersByLength[Info.Name.Len()];
		UniformBuffersByLength[Info.Name.Len()] = InfoIndex;
	}
	else
	{
		// If no members, we don't care about it
		UniformBufferInfos.RemoveAt(UniformBufferInfos.Num() - 1);
	}

	return EndPtr;
}

enum class AsciiFlags
{
	TerminatorOrSlash = (1 << 0),	// Null terminator OR slash (latter we care about for skipping commented out uniform blocks)
	Whitespace = (1 << 1),			// Includes other special characters below 32 (in addition to tab / newline)
	Other = (1 << 2),				// Anything else not one of the other types
	SymbolStart = (1<<3),			// Letters plus underscore (anything that can start a symbol)
	Digit = (1 << 4),
	Dot = (1 << 5),
	Quote = (1 << 6),
	Hash = (1 << 7),
};

static uint8 AsciiFlagTable[256] =
{
	1,2,2,2,2,2,2,2, 2,2,2,2,2,2,2,2, 2,2,2,2,2,2,2,2, 2,2,2,2,2,2,2,2,		// Treat all special characters as whitespace

	2,4,64,128,4,4,4,4,			// 34 == Quote  35 == Hash
	4,4,4,4,4,4,32,1,			// 46 == Dot    47 == Slash
	16,16,16,16,16,16,16,16,	// Digits 0-7
	16,16,4,4,4,4,4,4,			// Digits 8-9

	4,8,8,8,8,8,8,8, 8,8,8,8,8,8,8,8, 8,8,8,8,8,8,8,8, 8,8,8,4,4,4,4,8,		// Upper case letters,  95 == Underscore
	4,8,8,8,8,8,8,8, 8,8,8,8,8,8,8,8, 8,8,8,8,8,8,8,8, 8,8,8,4,4,4,4,4,		// Lower case letters

	4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4,		// Treat all non-ASCII characters as Other
	4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4,
	4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4,
	4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4, 4,4,4,4,4,4,4,4,
};

struct FCompoundIdentifierResult
{
	const FShaderSource::CharType* Identifier;			// Start of identifier
	const FShaderSource::CharType* IdentifierEnd;			// End of entire identifier
	const FShaderSource::CharType* IdentifierRootEnd;		// End of root token of identifier
};

// Searches for a "compound identifier" (series of symbol tokens separated by dots) that also passes the "RootIdentifierFilter".
// The filter is a mask table of valid identifier start characters indexed by identifier length.  Since identifier characters start
// with letters or underscore, we can store a 64-bit mask representing ASCII characters 64..127, as all valid start characters are
// in that range.  As an example, if "View" is a valid root identifier, RootIdentifierFilter[4] will have the bit ('V' - 64) set,
// and any other 4 character identifier that doesn't start with that letter can be skipped, saving overhead in the caller.
bool FindNextCompoundIdentifier(const FShaderSource::CharType*& Search, const uint64 RootIdentifierFilter[64], FCompoundIdentifierResult& OutResult)
{
	const FShaderSource::CharType* SearchChar = Search;
	uint8 SearchCharFlag = AsciiFlagTable[(uint8)*SearchChar];

	// Scanning loop
	while (1)
	{
		static constexpr uint8 AsciiFlagsEchoVerbatim = (uint8)AsciiFlags::Whitespace | (uint8)AsciiFlags::Other;
		static constexpr uint8 AsciiFlagsSymbol = (uint8)AsciiFlags::SymbolStart | (uint8)AsciiFlags::Digit;
		static constexpr uint8 AsciiFlagsStartNumberOrDirective = (uint8)AsciiFlags::Digit | (uint8)AsciiFlags::Dot | (uint8)AsciiFlags::Hash;
		static constexpr uint8 AsciiFlagsEndNumberOrDirective = (uint8)AsciiFlags::Whitespace | (uint8)AsciiFlags::Other | (uint8)AsciiFlags::Quote | (uint8)AsciiFlags::TerminatorOrSlash;

		// Conditions here are organized in expected order of frequency
		if (SearchCharFlag & AsciiFlagsEchoVerbatim)
		{
			SearchChar++;
			SearchCharFlag = AsciiFlagTable[(uint8)*SearchChar];
		}
		else if (SearchCharFlag & (uint8)AsciiFlags::SymbolStart)
		{
			OutResult.Identifier = SearchChar;
			SearchChar++;
			while ((SearchCharFlag = AsciiFlagTable[(uint8)*SearchChar]) & AsciiFlagsSymbol)
			{
				SearchChar++;
			}

			// Track end of our root identifier
			OutResult.IdentifierRootEnd = SearchChar;

			// Skip any whitespace before a potential dot
			while (SearchCharFlag & ((uint8)AsciiFlags::Whitespace))
			{
				SearchChar++;
				SearchCharFlag = AsciiFlagTable[(uint8)*SearchChar];
			}

			// If we didn't find a dot, go back to initial scanning state
			if (!(SearchCharFlag & ((uint8)AsciiFlags::Dot)))
			{
				continue;
			}
			SearchChar++;
			SearchCharFlag = AsciiFlagTable[(uint8)*SearchChar];

			// Determine if this root identifier passes the filter.  If so, we'll continue to parse the rest of the identifier,
			// but then go back to scanning.  The mask in RootIdentifierFilter starts with ASCII character 64, as token start
			// characters are in the range [64..127].
			ptrdiff_t IdentifierRootLen = OutResult.IdentifierRootEnd - OutResult.Identifier;
			if (IdentifierRootLen >= 64 || !(RootIdentifierFilter[IdentifierRootLen] & (1ull << (*OutResult.Identifier - 64))))
			{
				// Clear this, marking that we didn't find a candidate root identifier
				OutResult.IdentifierRootEnd = nullptr;
			}

			// Skip any whitespace after dot
			while (SearchCharFlag & ((uint8)AsciiFlags::Whitespace))
			{
				SearchChar++;
				SearchCharFlag = AsciiFlagTable[(uint8)*SearchChar];
			}

			// Check for the start of another symbol after the dot -- if it's not a symbol, switch back to scanning -- some kind of incorrect code
			if (!(SearchCharFlag & (uint8)AsciiFlags::SymbolStart))
			{
				continue;
			}

			// Repeatedly scan for additional parts of the identifier separated by dots
			while (1)
			{
				SearchChar++;
				while ((SearchCharFlag = AsciiFlagTable[(uint8)*SearchChar]) & AsciiFlagsSymbol)
				{
					SearchChar++;
				}

				// Track that this may be the end of the identifier (if there's not more dot separated tokens)
				OutResult.IdentifierEnd = SearchChar;

				// Skip any whitespace before a potential dot
				while (SearchCharFlag & ((uint8)AsciiFlags::Whitespace))
				{
					SearchChar++;
					SearchCharFlag = AsciiFlagTable[(uint8)*SearchChar];
				}

				// If we found something other than a dot, we're done!
				if (!(SearchCharFlag & ((uint8)AsciiFlags::Dot)))
				{
					// Is the root token for this identifier a candidate based on the filter?
					if (OutResult.IdentifierRootEnd)
					{
						Search = SearchChar;
						return true;
					}
					else
					{
						// If not, go back to initial scanning state
						break;
					}
				}

				// Skip the dot
				SearchChar++;
				SearchCharFlag = AsciiFlagTable[(uint8)*SearchChar];

				// Skip any whitespace after dot
				while (SearchCharFlag & ((uint8)AsciiFlags::Whitespace))
				{
					SearchChar++;
					SearchCharFlag = AsciiFlagTable[(uint8)*SearchChar];
				}

				// Did we find the start of another symbol after the dot?  If not, break out, some kind of invalid code...
				if (!(SearchCharFlag & (uint8)AsciiFlags::SymbolStart))
				{
					break;
				}
			}
		}
		else if (SearchCharFlag & AsciiFlagsStartNumberOrDirective)
		{
			// Number or directive, skip to Whitespace, Other, or Quote (numbers may contain letters or #, i.e. "1.#INF" for infinity, or "e" for an exponent)
			SearchChar++;
			while (!((SearchCharFlag = AsciiFlagTable[(uint8)*SearchChar]) & AsciiFlagsEndNumberOrDirective))
			{
				SearchChar++;
			}
		}
		else if (SearchCharFlag & (uint8)AsciiFlags::Quote)
		{
			// Quote, skip to next Quote (or maybe end of string if text is malformed), ignoring the quote if it's escaped
			SearchChar++;
			while (*SearchChar && (*SearchChar != '\"' || *(SearchChar - 1) == '\\'))
			{
				SearchChar++;
			}

			// Could be end of string or the quote -- skip over the quote if not the null terminator
			if (*SearchChar)
			{
				SearchChar++;
			}
			SearchCharFlag = AsciiFlagTable[(uint8)*SearchChar];
		}
		// Must be null terminator or slash at this point -- we've tested all other possibilities
		else if (*SearchChar == '/')
		{
			// Check if this is a commented out block (typically a commented out uniform declaration) and skip over it.
			// If the text is bad, there could be a /* right at the end of the string, so we need to check there is at least
			// one more character.
			if (SearchChar[1] == '*' && SearchChar[2] != 0)
			{
				// Search for slash (or end of string), starting at SearchChar + 3.  If we find a slash, we'll check the previous
				// character to see if it's the end of the comment.  Starting at +3 is necessary to avoid matching a slash as the
				// first character of the comment, i.e. "/*/".
				SearchChar += 3;

				while (1)
				{
					while ((SearchCharFlag = AsciiFlagTable[(uint8)*SearchChar]) != (uint8)AsciiFlags::TerminatorOrSlash)
					{
						SearchChar++;
					}

					// Is this the end of the comment?
					if (*(SearchChar - 1) == '*')
					{
						if (*SearchChar)
						{
							SearchChar++;
							SearchCharFlag = AsciiFlagTable[(uint8)*SearchChar];
							break;
						}
					}
					else
					{
						// More characters, continue the comment scanning loop, or if somehow at end of string, return false...
						if (*SearchChar)
						{
							SearchChar++;
						}
						else
						{
							return false;
						}
					}
				}
			}
			else
			{
				// Just a slash, not part of a block comment
				SearchChar++;
				SearchCharFlag = AsciiFlagTable[(uint8)*SearchChar];
			}
		}
		else
		{
			// End of string
			Search = SearchChar;
			return false;
		}
	}
}

FShaderSource::CharType* FindNextUniformBufferDefinition(FShaderSource::CharType* SearchPtr, FShaderSource::CharType* SourceStart, FShaderSource::FViewType UniformBufferStructIdentifier)
{
	while (SearchPtr)
	{
		SearchPtr = FShaderSource::FCStringType::Strstr(SearchPtr, UniformBufferStructIdentifier.GetData());

		if (SearchPtr)
		{
			if (SearchPtr > SourceStart && IsSpaceOrTabOrEOL(*(SearchPtr - 1)) && IsSpaceOrTabOrEOL(*(SearchPtr + UniformBufferStructIdentifier.Len())))
			{
				break;
			}
			else
			{
				SearchPtr = SearchPtr + 1;
			}
		}
	}
	return SearchPtr;
}

const FShaderSource::CharType* FindPreviousDot(const FShaderSource::CharType* SearchPtr, const FShaderSource::CharType* SearchMin)
{
	while ((SearchPtr > SearchMin) && (*SearchPtr != '.'))
	{
		SearchPtr--;
	}
	return SearchPtr;
}

// The cross compiler doesn't yet support struct initializers needed to construct static structs for uniform buffers
// Replace all uniform buffer struct member references (View.WorldToClip) with a flattened name that removes the struct dependency (View_WorldToClip)
void CleanupUniformBufferCode(const FShaderCompilerEnvironment& Environment, FShaderSource& PreprocessedShaderSource)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CleanupUniformBufferCode);

	TArray<FUniformBufferInfoNew> UniformBufferInfos;
	TArray<FUniformBufferSpan> UniformBufferSpans;
	uint64 UniformBufferFilter[64] = { 0 };			// A bit set for valid start characters for uniform buffer name of given length
	int32 UniformBuffersByLength[64];				// Linked list head index into UniformBufferInfos by length (connected by "NextWithSameLength")

	UniformBufferInfos.Reserve(Environment.UniformBufferMap.Num());
	UniformBufferSpans.Reserve(Environment.UniformBufferMap.Num());
	memset(UniformBuffersByLength, 0xff, sizeof(UniformBuffersByLength));

	FShaderSource::FViewType UniformBufferStructIdentifier = ANSITEXTVIEW("UniformBuffer");

	FShaderSource::CharType* SourceStart = PreprocessedShaderSource.GetData();
	FShaderSource::CharType* SearchPtr = SourceStart;
	FShaderSource::CharType* EndOfPreviousUniformBuffer = SourceStart;
	bool bUniformBufferFound;

	do
	{
		// Find the next uniform buffer definition
		SearchPtr = FindNextUniformBufferDefinition(SearchPtr, SourceStart, UniformBufferStructIdentifier);

		if (SearchPtr)
		{
			// Track that we found a uniform buffer, and temporarily null terminate the string so we can parse to this point
			bUniformBufferFound = true;
			*SearchPtr = 0;
		}
		else
		{
			bUniformBufferFound = false;
		}

		// Parse the source between the last uniform buffer and the current uniform buffer (or potentially the end of the source if no more
		// were found).  If there are no uniform buffers yet, we don't need to parse anything.
		if (UniformBufferInfos.Num())
		{
			const FShaderSource::CharType* ParsePtr = EndOfPreviousUniformBuffer;

			FCompoundIdentifierResult Result;
			while (FindNextCompoundIdentifier(ParsePtr, UniformBufferFilter, Result))
			{
				// Check if the identifier corresponds to a uniform buffer
				FShaderSource::FViewType IdentifierRoot(Result.Identifier, Result.IdentifierRootEnd - Result.Identifier);
				for (int32 UniformInfoIndex = UniformBuffersByLength[IdentifierRoot.Len()]; UniformInfoIndex != INDEX_NONE; UniformInfoIndex = UniformBufferInfos[UniformInfoIndex].NextWithSameLength)
				{
					FUniformBufferInfoNew& Info = UniformBufferInfos[UniformInfoIndex];
					if (IdentifierRoot.Equals(Info.Name, ESearchCase::CaseSensitive))
					{
						// Found the uniform buffer, clean up potential whitespace
						Result.IdentifierEnd = CompactCompoundIdentifier(const_cast<FShaderSource::CharType*>(Result.Identifier), const_cast<FShaderSource::CharType*>(Result.IdentifierEnd));

						// Now try to find a matching member.  We need to check subsets of the full "identifier", to strip away function calls, components, or child structures.
						bool bMatchFound = false;

						for (; Result.IdentifierEnd > Result.IdentifierRootEnd; Result.IdentifierEnd = FindPreviousDot(Result.IdentifierEnd - 1, Result.IdentifierRootEnd))
						{
							FShaderSource::FViewType Identifier(Result.Identifier, Result.IdentifierEnd - Result.Identifier);
							if (Identifier.Len() < Info.MembersByLength.Num())
							{
								const FUniformBufferMemberView& MemberView = Info.MembersByLength[Identifier.Len()];

								for (int32 MemberIndex = MemberView.MemberOffset; MemberIndex < MemberView.MemberOffset + MemberView.MemberCount; MemberIndex++)
								{
									if (Info.Members[MemberIndex].NameAsStructMember.Equals(Identifier, ESearchCase::CaseSensitive))
									{
										bMatchFound = true;

										const int32 OriginalTextLen = Info.Members[MemberIndex].NameAsStructMember.Len();
										const int32 ReplacementTextLen = Info.Members[MemberIndex].GlobalName.Len();

										const FShaderSource::CharType* GlobalNameStart = GetData(Info.Members[MemberIndex].GlobalName);
										FShaderSource::CharType* IdentifierStart = const_cast<FShaderSource::CharType*>(Result.Identifier);

										int32 Index = 0;
										for (; Index < ReplacementTextLen; Index++)
										{
											IdentifierStart[Index] = GlobalNameStart[Index];
										}
										for (; Index < OriginalTextLen; Index++)
										{
											IdentifierStart[Index] = ' ';
										}
										break;
									}
								}

								if (bMatchFound)
								{
									break;
								}
							}
						}

						break;
					}
				}
			}
		}

		// Parse the current uniform buffer.
		if (bUniformBufferFound)
		{
			// Unterminate the string (put the first character of the struct identifier back in place) and parse it
			*SearchPtr = UniformBufferStructIdentifier[0];

			const FShaderSource::CharType* ConstStructEndPtr = ParseUniformBufferDefinition(SearchPtr + UniformBufferStructIdentifier.Len(), UniformBufferInfos, UniformBufferFilter, UniformBuffersByLength);
			FShaderSource::CharType* StructEndPtr = &SourceStart[ConstStructEndPtr - &SourceStart[0]];

			// Comment out the uniform buffer struct and initializer
			*SearchPtr = '/';
			*(SearchPtr + 1) = '*';
			*(StructEndPtr - 1) = '*';
			*StructEndPtr = '/';

			UniformBufferSpans.Add({ (int32)(SearchPtr - SourceStart), (int32)(StructEndPtr + 1 - SearchPtr) });

			EndOfPreviousUniformBuffer = StructEndPtr + 1;
			SearchPtr = StructEndPtr + 1;
		}

	} while (bUniformBufferFound);

	// Uncomment this if to keep the UniformBuffer{} definitions in the preprocessed code.
	//if (false)
	{
		// Compact commented out uniform buffers out of the output source.  This costs around 10x less to do here than later in the minifier.  Note that
		// it's not necessary to add a line directive to fix up line numbers because uniform buffer declarations are always in generated files, and there
		// will be a line directive already there for the transition from the generated file back to whatever file included it.  The destination offset
		// for the first move is the start of the first uniform buffer declaration we are overwriting, then advances as characters are copied.
		int32 DestOffset = UniformBufferSpans.Num() ? UniformBufferSpans[0].Offset : PreprocessedShaderSource.Len();

		for (int32 SpanIndex = 0; SpanIndex < UniformBufferSpans.Num(); SpanIndex++)
		{
			// The source code we are compacting down is from the end of one span to the start of the next span, or end of the string.
			// We do not need to account for null terminator as the ShrinkToLen call below will null terminate for us.
			int32 SourceOffset = UniformBufferSpans[SpanIndex].Offset + UniformBufferSpans[SpanIndex].Length;
			int32 MoveCount = (SpanIndex < UniformBufferSpans.Num() - 1 ? UniformBufferSpans[SpanIndex + 1].Offset : PreprocessedShaderSource.Len()) - SourceOffset;

			check(DestOffset >= 0 && DestOffset < SourceOffset && SourceOffset + MoveCount <= PreprocessedShaderSource.Len());

			memmove(SourceStart + DestOffset, SourceStart + SourceOffset, MoveCount * sizeof(FShaderSource::CharType));
			DestOffset += MoveCount;
		}
		PreprocessedShaderSource.ShrinkToLen(DestOffset, EAllowShrinking::No);
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static FString CreateShaderConductorCommandLine(const FShaderCompilerInput& Input, const FString& SourceFilename, EShaderConductorTarget SCTarget)
{
	const TCHAR* Stage = nullptr;
	switch (Input.Target.GetFrequency())
	{
	case SF_Vertex:			Stage = TEXT("vs"); break;
	case SF_Pixel:			Stage = TEXT("ps"); break;
	case SF_Geometry:		Stage = TEXT("gs"); break;
	case SF_Compute:		Stage = TEXT("cs"); break;
	default:				return FString();
	}

	const TCHAR* Target = nullptr;
	switch (SCTarget)
	{
	case EShaderConductorTarget::Dxil:		Target = TEXT("dxil"); break;
	case EShaderConductorTarget::Spirv:		Target = TEXT("spirv"); break;
	default:								return FString();
	}

	FString CmdLine = TEXT("-E ") + Input.EntryPointName;
	//CmdLine += TEXT("-O ") + *(CompilerInfo.Input.D);
	CmdLine += TEXT(" -S ") + FString(Stage);
	CmdLine += TEXT(" -T ");
	CmdLine += Target;
	CmdLine += TEXT(" -I ") + (Input.DumpDebugInfoPath / SourceFilename);

	return CmdLine;
}

SHADERCOMPILERCOMMON_API void WriteShaderConductorCommandLine(const FShaderCompilerInput& Input, const FString& SourceFilename, EShaderConductorTarget Target)
{
	FArchive* FileWriter = IFileManager::Get().CreateFileWriter(*(Input.DumpDebugInfoPath / TEXT("ShaderConductorCmdLine.txt")));
	if (FileWriter)
	{
		FString CmdLine = CreateShaderConductorCommandLine(Input, SourceFilename, Target);

		FileWriter->Serialize(TCHAR_TO_ANSI(*CmdLine), CmdLine.Len());
		FileWriter->Close();
		delete FileWriter;
	}
}

static uint32 OfflineCompiler_ExtractStats(const FString& CompilerOutput, const TArray<FString>& InstructionStrings)
{
	uint32 ReturnedNum = 0;

	// Parse the instruction count
	int32 InstructionStringLength = 0, InstructionsIndex = 0;
	for (const FString& InstrStr : InstructionStrings)
	{
		InstructionStringLength = InstrStr.Len();
		InstructionsIndex = CompilerOutput.Find(*InstrStr);
		if (InstructionsIndex != INDEX_NONE)
		{
			break;
		}
	}

	if (InstructionsIndex != INDEX_NONE && InstructionsIndex + InstructionStringLength < CompilerOutput.Len())
	{
		const int32 EndIndex = CompilerOutput.Find(TEXT("\n"), ESearchCase::IgnoreCase, ESearchDir::FromStart, InstructionsIndex + InstructionStringLength);

		if (EndIndex != INDEX_NONE)
		{
			int32 StartIndex = InstructionsIndex + InstructionStringLength;

			bool bFoundNrStart = false;
			int32 NumberIndex = 0;

			while (StartIndex < EndIndex)
			{
				if (FChar::IsDigit(CompilerOutput[StartIndex]) && !bFoundNrStart)
				{
					// found number's beginning
					bFoundNrStart = true;
					NumberIndex = StartIndex;
				}
				else if (FChar::IsWhitespace(CompilerOutput[StartIndex]) && bFoundNrStart)
				{
					// found number's end
					bFoundNrStart = false;
					const FString NumberString = CompilerOutput.Mid(NumberIndex, StartIndex - NumberIndex);
					const float fNrInstructions = FCString::Atof(*NumberString);
					ReturnedNum += (uint32)FMath::Max(0.0, ceil(fNrInstructions));
				}

				++StartIndex;
			}
		}
	}

	return ReturnedNum;
}

static FString OfflineCompiler_ExtractErrors(const FString& CompilerOutput)
{
	FString ReturnedErrors;

	const int32 GlobalErrorIndex = CompilerOutput.Find(TEXT("Compilation failed."));

	// find each 'line' that begins with token "ERROR:" and copy it to the returned string
	if (GlobalErrorIndex != INDEX_NONE)
	{
		int32 CompilationErrorIndex = CompilerOutput.Find(TEXT("ERROR:"));
		while (CompilationErrorIndex != INDEX_NONE)
		{
			int32 EndLineIndex = CompilerOutput.Find(TEXT("\n"), ESearchCase::CaseSensitive, ESearchDir::FromStart, CompilationErrorIndex + 1);
			EndLineIndex = EndLineIndex == INDEX_NONE ? CompilerOutput.Len() - 1 : EndLineIndex;

			ReturnedErrors += CompilerOutput.Mid(CompilationErrorIndex, EndLineIndex - CompilationErrorIndex + 1);

			CompilationErrorIndex = CompilerOutput.Find(TEXT("ERROR:"), ESearchCase::CaseSensitive, ESearchDir::FromStart, EndLineIndex);
		}
	}

	return ReturnedErrors;
}

/**
* OfflineShaderCompiler's compilation command line options.
* Each OfflineShaderCompiler should specify its own FOfflineShaderCompilerOptions. If one option is not supported by this OfflineShaderCompiler, leave it empty.
* Frequency (VS/PS/etc.) here is called as stage sometime, too.
*/
class FOfflineShaderCompilerOptions
{
public:
	/** Options applied to all shaders. */
	FString CommonOptions;
	/** MultiView option if it's enabled. */
	FString MultiViewOption;
	/** GPUTarget option*/
	FString GPUTargetOption;
	/** Default GPUTarget*/
	FString DefaultGPUTarget;
	/** Dump All*/
	FString DumpAll;
	/** SpirV file extension name*/
	FString SpirVExt;
	/** Default file extension name*/
	FString DefaultGLSLExt;
	/** GLSL source file extension used to specify which shader stage is being compiled. */
	TMap<EShaderFrequency, FString> FrequencyGLSLExts;
	/** Option to specify which shader stage is being compiled. */
	TMap<EShaderFrequency, FString> FrequencyOptions;
	/** Entrypoint option used to specify the entry point of each shader frequency. */
	TMap<EShaderFrequency, FString> FrequencyEntryPoints;
	/** Extra option of each shader frequency. */
	TMap<EShaderFrequency, FString> FrequencyExtraOption;
	/** Entrypoint option used to specify the entry point of all shader frequencies. */
	TCHAR* DefaultEntryPoint;

	/** Used to parse stats output to find total instruction count. Using array to support multiple compiler versions*/
	TArray<FString> NumInstructionNames;
	/** Used to parse stats output to find each stat. Each item of this array is for one stat AND it is also an array to support multiple compiler versions. */
	TArray<TArray<FString>> StatsNames;

	static FString GetFrequenceName(EShaderFrequency Freq)
	{
		static FString FrequencyName[SF_NumFrequencies] =
		{
			TEXT("VS"), // SF_Vertex
			TEXT("MS"), // SF_Mesh
			TEXT("AS"), // SF_Amplification
			TEXT("FS"), // SF_Pixel
			TEXT("GS"), // SF_Geometry
			TEXT("CS")  // SF_Compute
		};
		if (Freq <= SF_Compute)
		{
			return FrequencyName[Freq];
		}
		else
		{
			return FString("Unknown");
		}
	}
};

void CompileShaderOffline(const FShaderCompilerInput& Input,
	FShaderCompilerOutput& ShaderOutput,
	const ANSICHAR* ShaderSource,
	const int32 SourceSize,
	bool bVulkanSpirV,
	const FOfflineShaderCompilerOptions& Options,
	const ANSICHAR* VulkanSpirVEntryPoint)
{
	const auto Frequency = (EShaderFrequency)Input.Target.Frequency;
	const FString WorkingDir(FPlatformProcess::ShaderDir());

	FString CompilerPath = Input.ExtraSettings.OfflineCompilerPath;

	// add process and thread ids to the file name to avoid collision between workers
	auto ProcID = FPlatformProcess::GetCurrentProcessId();
	auto ThreadID = FPlatformTLS::GetCurrentThreadId();

	auto GetFileName = [&](FString FileType, FString Ext, int NumInst = 0)
	{
		return WorkingDir
			/ FOfflineShaderCompilerOptions::GetFrequenceName(Frequency) + (NumInst ? TEXT("-") + FString::FromInt(NumInst) : TEXT(""))
			+ FString::Printf(TEXT("-%s"), (VulkanSpirVEntryPoint ? ANSI_TO_TCHAR(VulkanSpirVEntryPoint) : TEXT("")))
			+ *FileType + TEXT("-") + FString::FromInt(ProcID) + TEXT("-") + FString::FromInt(ThreadID)
			+ *Ext;
	};

	FString ShaderSrcExt;
	if (bVulkanSpirV)
	{
		ShaderSrcExt = Options.SpirVExt;
	}
	else
	{
		if (const FString* Ext = Options.FrequencyGLSLExts.Find(Frequency))
		{
			ShaderSrcExt = *Ext;
		}
		else
		{
			ShaderSrcExt = Options.DefaultGLSLExt;
		}
	}
	
	FString ShaderSourceFile = GetFileName(FString("-Source"), ShaderSrcExt);
	FString CompilerCommand = Options.CommonOptions;
	if (!Options.GPUTargetOption.IsEmpty())
	{
		FString GPUTarget = Input.ExtraSettings.GPUTarget;
		if (GPUTarget.IsEmpty())
		{
			GPUTarget = Options.DefaultGPUTarget;			
		}
		CompilerCommand += FString::Printf(TEXT("%s=%s"), *Options.GPUTargetOption, *GPUTarget);
	}
	
	if (Input.ExtraSettings.bMobileMultiView)
	{
		CompilerCommand += Options.MultiViewOption;
	}

	CompilerCommand += Options.FrequencyOptions[Frequency];

	if (bVulkanSpirV)
	{
		CompilerCommand += FString::Printf(TEXT("%s %s"), *Options.FrequencyEntryPoints[Frequency], ANSI_TO_TCHAR(VulkanSpirVEntryPoint));
	}

	if (const FString* ExtraOption = Options.FrequencyExtraOption.Find(Frequency)) 
	{
		CompilerCommand += *ExtraOption;
	}

	FArchive* Ar = IFileManager::Get().CreateFileWriter(*ShaderSourceFile, FILEWRITE_EvenIfReadOnly);

		if (Ar == nullptr)
		{
			return;
		}

		// write out the shader source to a file and use it below as input for the compiler
		Ar->Serialize((void*)ShaderSource, SourceSize);
		delete Ar;

		FString StdOut;
		FString StdErr;
		int32 ReturnCode = 0;

		// Since v6.2.0, Mali compiler needs to be started in the executable folder or it won't find "external/glslangValidator" for Vulkan
		FString CompilerWorkingDirectory = FPaths::GetPath(CompilerPath);

	if (!CompilerWorkingDirectory.IsEmpty() && FPaths::DirectoryExists(CompilerWorkingDirectory))
	{
		// compiler command line contains flags and the GLSL source file name
		CompilerCommand += " " + FPaths::ConvertRelativePathToFull(ShaderSourceFile);

		// Run shader compiler and wait for completion
		FPlatformProcess::ExecProcess(*CompilerPath, *CompilerCommand, &ReturnCode, &StdOut, &StdErr, *CompilerWorkingDirectory);
	}
	else
	{
		StdErr = "Couldn't find offline compiler at " + CompilerPath;
	}

		// parse Mali's output and extract instruction count or eventual errors
		ShaderOutput.bSucceeded = (ReturnCode >= 0);
		if (ShaderOutput.bSucceeded)
		{
			// check for errors
			if (StdErr.Len())
			{
				ShaderOutput.bSucceeded = false;

				FShaderCompilerError& NewError = ShaderOutput.Errors.AddDefaulted_GetRef();
				NewError.StrippedErrorMessage = TEXT("[Offline Complier]\n") + StdErr;
			}
			else
			{
				FString Errors = OfflineCompiler_ExtractErrors(StdOut);

				if (Errors.Len())
				{
					FShaderCompilerError& NewError = ShaderOutput.Errors.AddDefaulted_GetRef();
					NewError.StrippedErrorMessage = TEXT("[Offline Complier]\n") + Errors;
					ShaderOutput.bSucceeded = false;
				}
			}

			// extract instruction count
			if (ShaderOutput.bSucceeded)
			{
				ShaderOutput.NumInstructions = OfflineCompiler_ExtractStats(StdOut, Options.NumInstructionNames);
				FString OutputStatsFile = GetFileName(FString("-Stats"), FString(".txt"), ShaderOutput.NumInstructions);
				for (auto& StatNames : Options.StatsNames)
				{
					if(StatNames.Num())
					{
						ShaderOutput.AddStatistic<uint32>(*StatNames[0], OfflineCompiler_ExtractStats(StdOut, StatNames));
					}
				}
				if (Input.ExtraSettings.bSaveCompilerStatsFiles)
				{
					FArchive* ArOutput = IFileManager::Get().CreateFileWriter(*OutputStatsFile, FILEWRITE_EvenIfReadOnly);
					if (ArOutput == nullptr)
					{
						return;
					}
					if (!Options.DumpAll.IsEmpty())
					{
						CompilerCommand += Options.DumpAll;
						//TODO: It's expensive to run the process twice. Better to run it once with DumpAll and parse the StdOut to get Stats.
						// But to do that, we need to know the preserved keyword for Stats.
						FPlatformProcess::ExecProcess(*CompilerPath, *CompilerCommand, &ReturnCode, &StdOut, &StdErr, *CompilerWorkingDirectory);
					}
					FString StatsOutput = CompilerCommand + FString("\n") + StdOut;
					const int32 StatsLen = StatsOutput.Len();
					TSharedPtr<ANSICHAR> ShaderStats = MakeShareable(new ANSICHAR[StatsLen + 1]);
					FCStringAnsi::Strncpy(ShaderStats.Get(), TCHAR_TO_ANSI(*StatsOutput), StatsLen + 1);
					ArOutput->Serialize((void*)ShaderStats.Get(), StatsLen);
					delete ArOutput;
				}
			}
	}

		// we're done so delete the shader file
	if (Input.ExtraSettings.bSaveCompilerStatsFiles)
	{
		FString DstShaderSourceFile = GetFileName(FString("-Source"), ShaderSrcExt, ShaderOutput.NumInstructions);
		IFileManager::Get().Move(*DstShaderSourceFile, *ShaderSourceFile, true, true);
		IFileManager::Get().Delete(*ShaderSourceFile, true, true);
	}
	IFileManager::Get().Delete(*ShaderSourceFile, true, true);
}

void CompileShaderOffline_Mali(const FShaderCompilerInput& Input,
	FShaderCompilerOutput& ShaderOutput,
	const ANSICHAR* ShaderSource,
	const int32 SourceSize,
	bool bVulkanSpirV,
	const ANSICHAR* VulkanSpirVEntryPoint)
{
	static FOfflineShaderCompilerOptions Options;
	if (bVulkanSpirV)
	{
		Options.CommonOptions = TEXT(" -p");
	}
	else
	{
		Options.CommonOptions = TEXT(" -s");
	}

	if (Options.SpirVExt.IsEmpty())
	{
		Options.SpirVExt = TEXT(".spv");
		Options.DefaultGLSLExt = TEXT(".shd");
		Options.FrequencyGLSLExts.Emplace(SF_Vertex, TEXT(".vert"));
		Options.FrequencyGLSLExts.Emplace(SF_Pixel, TEXT(".frag"));
		Options.FrequencyGLSLExts.Emplace(SF_Geometry, TEXT(".geom"));
		Options.FrequencyGLSLExts.Emplace(SF_Compute, TEXT(".comp"));

		Options.FrequencyOptions.Emplace(SF_Vertex, FString(" -v"));
		Options.FrequencyOptions.Emplace(SF_Pixel, FString(" -f"));
		Options.FrequencyOptions.Emplace(SF_Geometry, FString(" -g"));
		Options.FrequencyOptions.Emplace(SF_Compute, FString(" -C"));

		Options.FrequencyEntryPoints.Emplace(SF_Vertex, TEXT(" -y"));
		Options.FrequencyEntryPoints.Emplace(SF_Pixel, TEXT(" -y"));
		Options.FrequencyEntryPoints.Emplace(SF_Geometry, TEXT(" -y"));
		Options.FrequencyEntryPoints.Emplace(SF_Compute, TEXT(" -y"));

		Options.NumInstructionNames.Add("Instructions Emitted:");
		Options.NumInstructionNames.Add("Total instruction cycles:");
	}

	CompileShaderOffline(Input, ShaderOutput, ShaderSource, SourceSize, bVulkanSpirV, Options, VulkanSpirVEntryPoint);
}

void CompileShaderOffline_Adreno(const FShaderCompilerInput& Input,
	FShaderCompilerOutput& ShaderOutput,
	const ANSICHAR* ShaderSource,
	const int32 SourceSize,
	bool bVulkanSpirV,
	const ANSICHAR* VulkanSpirVEntryPoint)
{
	static FOfflineShaderCompilerOptions Options;
	if (bVulkanSpirV)
	{
		Options.CommonOptions = TEXT(" -api=Vulkan");
	}

	if (Options.MultiViewOption.IsEmpty())
	{
		Options.MultiViewOption = TEXT(" -view_mask=0x3");
		Options.GPUTargetOption = TEXT(" -arch");
		Options.DefaultGPUTarget = TEXT("a650");

		Options.SpirVExt = TEXT(".spv");
		Options.DefaultGLSLExt = TEXT(".shd");
		Options.FrequencyGLSLExts.Emplace(SF_Vertex, TEXT(".vert"));
		Options.FrequencyGLSLExts.Emplace(SF_Pixel, TEXT(".frag"));
		Options.FrequencyGLSLExts.Emplace(SF_Geometry, TEXT(".geom"));
		Options.FrequencyGLSLExts.Emplace(SF_Compute, TEXT(".comp"));

		Options.FrequencyOptions.Emplace(SF_Vertex, FString(" -vs"));
		Options.FrequencyOptions.Emplace(SF_Pixel, FString(" -fs"));
		Options.FrequencyOptions.Emplace(SF_Geometry, FString(" -gs"));
		Options.FrequencyOptions.Emplace(SF_Compute, FString(" -cs"));

		Options.FrequencyEntryPoints.Emplace(SF_Vertex, TEXT(" -entry_point_vs"));
		Options.FrequencyEntryPoints.Emplace(SF_Pixel, TEXT(" -entry_point_ps"));
		Options.FrequencyEntryPoints.Emplace(SF_Geometry, TEXT(" -entry_point_gs"));
		Options.FrequencyEntryPoints.Emplace(SF_Compute, TEXT(" -entry_point_cs"));

		Options.FrequencyExtraOption.Emplace(SF_Vertex, TEXT(" -link_with_fs"));

		Options.NumInstructionNames.Add("Total instruction count");
		
		Options.StatsNames.Add(TArray<FString>{TEXT("ALU instruction count - 32 bit")});
		Options.StatsNames.Add(TArray<FString>{TEXT("ALU instruction count - 16 bit")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Complex instruction count - 32 bit")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Complex instruction count - 16 bit")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Flow control instruction count")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Barrier and fence Instruction count")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Short latency sync instruction count")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Long latency sync instruction count")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Texture read instruction count")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Memory read instruction count")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Memory write instruction count")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Miscellaneous instruction count")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Full precision register footprint per shader instance")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Half precision register footprint per shader instance")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Overall register footprint per shader instance")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Scratch memory usage per shader instance")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Loop count")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Output component count")});
		Options.StatsNames.Add(TArray<FString>{TEXT("Input component count")});
		Options.StatsNames.Add(TArray<FString>{TEXT("ALU fiber occupancy percentage")});
		
	}

	if (Input.ExtraSettings.bDumpAll) 
	{
		Options.DumpAll = " -dump=all";
	}

	CompileShaderOffline(Input, ShaderOutput, ShaderSource, SourceSize, bVulkanSpirV, Options, VulkanSpirVEntryPoint);
}

void CompileShaderOffline(const FShaderCompilerInput& Input,
	FShaderCompilerOutput& ShaderOutput,
	const ANSICHAR* ShaderSource,
	const int32 SourceSize,
	bool bVulkanSpirV,
	const ANSICHAR* VulkanSpirVEntryPoint)
{
	const bool bCompilerExecutableExists = FPaths::FileExists(Input.ExtraSettings.OfflineCompilerPath);
	if (!bCompilerExecutableExists)
	{
		return;
	}
	EOfflineShaderCompilerType OfflineCompiler = Input.ExtraSettings.OfflineCompiler;
	switch (OfflineCompiler)
	{
	case EOfflineShaderCompilerType::Mali:
		CompileShaderOffline_Mali(Input, ShaderOutput, ShaderSource, SourceSize, bVulkanSpirV, VulkanSpirVEntryPoint);
		break;
	case EOfflineShaderCompilerType::Adreno:
		CompileShaderOffline_Adreno(Input, ShaderOutput, ShaderSource, SourceSize, bVulkanSpirV, VulkanSpirVEntryPoint);
		break;
	case EOfflineShaderCompilerType::Num:
		break;
	default:
		break;
	}
}

// sensible default path size; TStringBuilder will allocate if it needs to
const FString GetDebugFileName(
	const FShaderCompilerInput& Input, 
	const UE::ShaderCompilerCommon::FDebugShaderDataOptions& Options, 
	const TCHAR* BaseFilename,
	const TCHAR* Suffix = nullptr)
{
	TStringBuilder<512> PathBuilder;
	const TCHAR* Prefix = (Options.FilenamePrefix && *Options.FilenamePrefix) ? Options.FilenamePrefix : TEXT("");
	FStringView Filename = (BaseFilename && *BaseFilename) ? BaseFilename : Input.GetSourceFilenameView();
	FStringView Ext = FPathViews::GetExtension(Filename, true);
	FStringView FilenameNoExt = Filename.LeftChop(Ext.Len());
	FPathViews::Append(PathBuilder, Input.DumpDebugInfoPath, Prefix);
	PathBuilder << FilenameNoExt;
	if (Suffix)
	{
		PathBuilder << Suffix;
	}
	PathBuilder << Ext;
	return PathBuilder.ToString();
}

namespace UE::ShaderCompilerCommon
{
	bool ExecuteShaderPreprocessingSteps(
		FShaderPreprocessOutput& PreprocessOutput,
		const FShaderCompilerInput& Input,
		const FShaderCompilerEnvironment& Environment,
		const FShaderCompilerDefinitions& AdditionalDefines
		)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FBaseShaderFormat_PreprocessShader);

		check(CheckVirtualShaderFilePath(Input.VirtualSourceFilePath));

		bool bSuccess = ::PreprocessShader(PreprocessOutput, Input, Environment, AdditionalDefines);
		if (bSuccess)
		{
			CleanupUniformBufferCode(Environment, PreprocessOutput.EditSource());

			if (Input.Environment.CompilerFlags.Contains(CFLAG_RemoveDeadCode))
			{
				const TArray<FStringView> RequiredSymbols(MakeArrayView(Input.RequiredSymbols));
				UE::ShaderCompilerCommon::RemoveDeadCode(PreprocessOutput.EditSource(), Input.EntryPointName, RequiredSymbols, PreprocessOutput.EditErrors());
			}
		}

		return bSuccess;
	}

	bool ExecuteShaderPreprocessingSteps(
		FShaderPreprocessOutput& PreprocessOutput,
		const FShaderCompilerInput& Input,
		const FShaderCompilerEnvironment& Environment
	)
	{
		// overloaded function rather than defaulting definitions parameter to avoid including internal header in public header
		return ExecuteShaderPreprocessingSteps(PreprocessOutput, Input, Environment, FShaderCompilerDefinitions());
	}

	FString FDebugShaderDataOptions::GetDebugShaderPath(const FShaderCompilerInput& Input, const TCHAR* Suffix) const
	{
		return GetDebugFileName(Input, *this, OverrideBaseFilename, Suffix);
	}

	bool FBaseShaderFormat::PreprocessShader(
		const FShaderCompilerInput& Input,
		const FShaderCompilerEnvironment& Environment,
		FShaderPreprocessOutput& PreprocessOutput) const
	{
		return ExecuteShaderPreprocessingSteps(PreprocessOutput, Input, Environment);
	}

	void FBaseShaderFormat::OutputDebugData(
		const FShaderCompilerInput& Input,
		const FShaderPreprocessOutput& PreprocessOutput,
		const FShaderCompilerOutput& Output,
		FShaderDebugDataContext& Ctx) const
	{
		DumpExtendedDebugShaderData(Input, PreprocessOutput, Output, Ctx);
	}

	static void DumpDebugShaderDataInternal(const FShaderCompilerInput& Input, FStringView PreprocessedSource, FShaderDebugDataContext& Ctx, const FDebugShaderDataOptions& Options, const TCHAR* Suffix)
	{
		if (!Input.DumpDebugInfoEnabled())
		{
			return;
		}

		FString Contents = UE::ShaderCompilerCommon::GetDebugShaderContents(Input, PreprocessedSource, Options, Suffix);
		FString DebugSourcePath = Options.GetDebugShaderPath(Input, Suffix);
		FFileHelper::SaveStringToFile(Contents, *DebugSourcePath);

		Ctx.DebugSourceFiles.Add({ Input.Target.GetFrequency(), MoveTemp(DebugSourcePath) });
	}

	void FBaseShaderFormat::OutputDebugDataMinimal(const FShaderCompilerInput& Input, FShaderDebugDataContext& Ctx) const
	{
		const FStringView FailedSourceStr = TEXTVIEW(R"(
// Preprocessing failed for shader; defines used can be seen above.
// ShaderCompileWorker can be run via the debugger using the commandline args in DebugCompileArgs.txt to inspect the contents of the FShaderCompilerInput/FShaderCompilerEnvironment.)");

		DumpDebugShaderDataInternal(Input, FailedSourceStr, Ctx, FDebugShaderDataOptions(), nullptr);
	}


	void DumpExtendedDebugShaderData(
		const FShaderCompilerInput& Input,
		const FShaderPreprocessOutput& PreprocessOutput,
		const FShaderCompilerOutput& Output,
		const FDebugShaderDataOptions& Options)
	{
		// deprecated
	}

	void DumpExtendedDebugShaderData(
		const FShaderCompilerInput& Input,
		const FShaderPreprocessOutput& PreprocessOutput,
		const FShaderCompilerOutput& Output,
		FShaderDebugDataContext& Ctx,
		const FDebugShaderDataOptions& Options)
	{
		if (!Input.Environment.CompilerFlags.Contains(CFLAG_DisableSourceStripping) && EnumHasAnyFlags(Input.DebugInfoFlags, EShaderDebugInfoFlags::DetailedSource))
		{
			const TCHAR* StrippedSuffix = TEXT("_Stripped");
			FFileHelper::SaveStringToFile(GetDebugShaderContents(Input, PreprocessOutput.GetSourceViewWide(), Options, StrippedSuffix), *Options.GetDebugShaderPath(Input, StrippedSuffix));
		}

		bool bHasModifiedSource = !Output.ModifiedShaderSource.IsEmpty();
		if (bHasModifiedSource)
		{
			// If the compile step applies modifications to the source, output this as the "default" USF; it's not compatible with SCW debug compile mode but backends
			// which output compile batch files rely on this being the copy of the source that can be passed directly to the platform compiler.
			FFileHelper::SaveStringToFile(Output.ModifiedShaderSource, *Options.GetDebugShaderPath(Input));
		}

		// if no modifications to source are made in the compile step, output just the single usf which is the unstripped version compatible with launching
		// SCW in debug compile mode (the stripped version is less useful for debugging via this mechanism, so is only output in "detailed source" mode)
		// if modifications were made, this is output as an additional artifact, appending "_DebugCompile" to the path to indicate that it can be used as such.
		DumpDebugShaderDataInternal(Input, PreprocessOutput.GetUnstrippedSourceView(), Ctx, Options, bHasModifiedSource ? TEXT("_DebugCompile") : nullptr);

		if (EnumHasAnyFlags(Input.DebugInfoFlags, EShaderDebugInfoFlags::ShaderCodePlatformHashes))
		{
			// if the platform has registered a CodeHash stat, output a file containing this as well
			const FGenericShaderStat* Hash = Output.ShaderStatistics.FindByPredicate([](const FGenericShaderStat& Stat)
				{
					return Stat.StatName == kPlatformHashStatName;
				});
			if (Hash)
			{
				FFileHelper::SaveStringToFile(Hash->Value.Get<FString>(), *GetDebugFileName(Input, Options, TEXT("PlatformHash.txt")), FFileHelper::EEncodingOptions::ForceAnsi);
			}
		}

		FFileHelper::SaveStringToFile(Output.OutputHash.ToString(), *GetDebugFileName(Input, Options, TEXT("OutputHash.txt")), FFileHelper::EEncodingOptions::ForceAnsi);

		if (EnumHasAnyFlags(Input.DebugInfoFlags, EShaderDebugInfoFlags::Diagnostics))
		{
			FString Merged;
			for (const FShaderCompilerError& Diag : Output.Errors)
			{
				Merged += Diag.GetErrorStringWithLineMarker() + "\n";
			}
			if (!Merged.IsEmpty())
			{
				FFileHelper::SaveStringToFile(Merged, *GetDebugFileName(Input, Options, TEXT("Diagnostics.txt")), FFileHelper::EEncodingOptions::ForceAnsi);
			}
		}
		
		// delete old DebugHash_* files so we don't clutter the debug info folder (these change every time the deadstripped source code changes)
		IFileManager::Get().IterateDirectory(*Input.DumpDebugInfoPath, [](const TCHAR* FilenameOrDirectory, bool bIsDirectory)
			{
				if (!bIsDirectory)
				{
					FStringView Filename = FPathViews::GetCleanFilename(FilenameOrDirectory);
					if (Filename.StartsWith(GetShaderSourceDebugHashPrefixWide()))
					{
						IFileManager::Get().Delete(FilenameOrDirectory);
					}
				}
				return true;
			});

		if (EnumHasAnyFlags(Input.DebugInfoFlags, EShaderDebugInfoFlags::InputHash) || Output.CompileTime > 0.0f)
		{
			// If compile time was > 0, this was the copy of the job that actually compiled; we write an empty file with the shader hash in it so it can be
			// found easily using the ShaderHash comment printed on the first line of the stripped source code.
			// We don't do this for cache hits or duplicate jobs unless explicitly requested (via the InputHash debug info flags), so that _all_ debug artifacts
			// (including those only generated by the compile process) are available in the folder containing this file.
			FString InputHashStr = LexToString(Input.Hash);
			FFileHelper::SaveStringToFile(FStringView(), *GetDebugFileName(Input, Options, GetShaderSourceDebugHashPrefixWide().GetData(), *InputHashStr));
		}

		if (EnumHasAnyFlags(Input.DebugInfoFlags, EShaderDebugInfoFlags::ShaderCodeBinary))
		{
			FString ShaderCodeFileName = *GetDebugFileName(Input, Options, TEXT("ShaderCode.bin"));
			if (Output.ShaderCode.IsCompressed())
			{
				// always output decompressed code as it's slightly more useful for A/B comparisons
				TArray<uint8> DecompressedCode;
				DecompressedCode.SetNum(Output.ShaderCode.GetUncompressedSize());
				bool bSucceed = FCompression::UncompressMemory(NAME_Oodle, DecompressedCode.GetData(), DecompressedCode.Num(), Output.ShaderCode.GetReadView().GetData(), Output.ShaderCode.GetShaderCodeSize());
				FFileHelper::SaveArrayToFile(DecompressedCode, *ShaderCodeFileName);
			}
			else
			{
				FFileHelper::SaveArrayToFile(Output.ShaderCode.GetReadView(), *ShaderCodeFileName);
			}
		}

		for (const FDebugShaderDataOptions::FAdditionalOutput& AdditionalOutput : Options.AdditionalOutputs)
		{
			FFileHelper::SaveStringToFile(AdditionalOutput.Data, *GetDebugFileName(Input, Options, AdditionalOutput.BaseFileName), FFileHelper::EEncodingOptions::ForceAnsi);
		}
	}

	static const TCHAR* Base64EnvBegin = TEXT("/* BASE64_ENV\n");
	static const int32 Base64EnvBeginLen = FCString::Strlen(Base64EnvBegin);
	static const TCHAR* Base64EnvEnd = TEXT("\nBASE64_ENV */\n");
	
	FString SerializeEnvironmentToBase64(const FShaderCompilerEnvironment& Env)
	{
		// deprecated
		static FString Empty;
		return Empty;
	}

	void SerializeEnvironmentFromBase64(FShaderCompilerEnvironment& Env, const FString& DebugShaderSource)
	{
		// deprecated
	}

	FString GetDebugShaderContents(const FShaderCompilerInput& Input, FStringView PreprocessedSource, const FDebugShaderDataOptions& Options, const TCHAR* Suffix)
	{
		// Debug dump occurs in the cook process, so we need to merge the env in Input.Environment with the shared env (this is done in the compile step as well)
		FShaderCompilerEnvironment MergedEnvironment(Input.Environment);
		if (IsValidRef(Input.SharedEnvironment))
		{
			MergedEnvironment.Merge(*Input.SharedEnvironment);
		}

		FString Contents = MergedEnvironment.GetDefinitionsAsCommentedCode();

		if (Options.AppendPreSource)
		{
			Contents += Options.AppendPreSource();
		}

		Contents += PreprocessedSource;

		if (Options.AppendPostSource)
		{
			Contents += Options.AppendPostSource();
		}

		Contents += TEXT("\n");
		if (!Input.DebugDescription.IsEmpty())
		{
			Contents += TEXT("//");
			Contents += Input.DebugDescription;
			Contents += TEXT("\n");
		}

		return Contents;
	}
}

void DumpDebugShaderText(const FShaderCompilerInput& Input, const FString& InSource, const FString& FileExtension)
{
	FTCHARToUTF8 StringConverter(InSource.GetCharArray().GetData(), InSource.Len());

	// Provide mutable container to pass string to FArchive inside inner function
	TArray<ANSICHAR> SourceAnsi;
	SourceAnsi.SetNum(InSource.Len() + 1);
	FCStringAnsi::Strncpy(SourceAnsi.GetData(), (ANSICHAR*)StringConverter.Get(), SourceAnsi.Num());

	// Forward temporary container to primary function
	DumpDebugShaderText(Input, SourceAnsi.GetData(), InSource.Len(), FileExtension);
}

void DumpDebugShaderText(const FShaderCompilerInput& Input, ANSICHAR* InSource, int32 InSourceLength, const FString& FileExtension)
{
	DumpDebugShaderBinary(Input, InSource, InSourceLength * sizeof(ANSICHAR), FileExtension);
}

void DumpDebugShaderText(const FShaderCompilerInput& Input, ANSICHAR* InSource, int32 InSourceLength, const FString& FileName, const FString& FileExtension)
{
	DumpDebugShaderBinary(Input, InSource, InSourceLength * sizeof(ANSICHAR), FileName, FileExtension);
}

void DumpDebugShaderBinary(const FShaderCompilerInput& Input, void* InData, int32 InDataByteSize, const FString& FileExtension)
{
	if (InData != nullptr && InDataByteSize > 0 && !FileExtension.IsEmpty())
	{
		const FString Filename = Input.DumpDebugInfoPath / FPaths::GetBaseFilename(Input.GetSourceFilename()) + TEXT(".") + FileExtension;
		if (TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*Filename)))
		{
			FileWriter->Serialize(InData, InDataByteSize);
			FileWriter->Close();
		}
	}
}

void DumpDebugShaderBinary(const FShaderCompilerInput& Input, void* InData, int32 InDataByteSize, const FString& FileName, const FString& FileExtension)
{
	if (InData != nullptr && InDataByteSize > 0 && !FileExtension.IsEmpty())
	{
		const FString Filename = Input.DumpDebugInfoPath / FileName + TEXT(".") + FileExtension;
		if (TUniquePtr<FArchive> FileWriter = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*Filename)))
		{
			FileWriter->Serialize(InData, InDataByteSize);
			FileWriter->Close();
		}
	}
}

static void DumpDebugShaderDisassembled(const FShaderCompilerInput& Input, CrossCompiler::EShaderConductorIR Language, void* InData, int32 InDataByteSize, const FString& FileExtension)
{
	if (InData != nullptr && InDataByteSize > 0 && !FileExtension.IsEmpty())
	{
		TArray<ANSICHAR> AssemblyText;
		if (CrossCompiler::FShaderConductorContext::Disassemble(Language, InData, InDataByteSize, AssemblyText))
		{
			// Assembly text contains NUL terminator, so text length is |array|-1
			DumpDebugShaderText(Input, AssemblyText.GetData(), AssemblyText.Num() - 1, FileExtension);
		}
	}
}

void DumpDebugShaderDisassembledSpirv(const FShaderCompilerInput& Input, void* InData, int32 InDataByteSize, const FString& FileExtension)
{
	DumpDebugShaderDisassembled(Input, CrossCompiler::EShaderConductorIR::Spirv, InData, InDataByteSize, FileExtension);
}

void DumpDebugShaderDisassembledDxil(const FShaderCompilerInput& Input, void* InData, int32 InDataByteSize, const FString& FileExtension)
{
	DumpDebugShaderDisassembled(Input, CrossCompiler::EShaderConductorIR::Dxil, InData, InDataByteSize, FileExtension);
}

namespace CrossCompiler
{
	/**
	 * Parse an error emitted by the HLSL cross-compiler.
	 * @param OutErrors - Array into which compiler errors may be added.
	 * @param InLine - A line from the compile log.
	 */
	void ParseHlslccError(TArray<FShaderCompilerError>& OutErrors, const FString& InLine, bool bUseAbsolutePaths)
	{
		const TCHAR* p = *InLine;
		FShaderCompilerError& Error = OutErrors.AddDefaulted_GetRef();

		// Copy the filename.
		while (*p && *p != TEXT('('))
		{
			Error.ErrorVirtualFilePath += (*p++);
		}

		if (!bUseAbsolutePaths)
		{
			Error.ErrorVirtualFilePath = ParseVirtualShaderFilename(Error.ErrorVirtualFilePath);
		}
		p++;

		// Parse the line number.
		int32 LineNumber = 0;
		while (*p && *p >= TEXT('0') && *p <= TEXT('9'))
		{
			LineNumber = 10 * LineNumber + (*p++ - TEXT('0'));
		}
		Error.ErrorLineString = *FString::Printf(TEXT("%d"), LineNumber);

		// Skip to the warning message.
		while (*p && (*p == TEXT(')') || *p == TEXT(':') || *p == TEXT(' ') || *p == TEXT('\t')))
		{
			p++;
		}
		Error.StrippedErrorMessage = p;
	}


	/** Map shader frequency -> string for messages. */
	static const TCHAR* FrequencyStringTable[] =
	{
		TEXT("Vertex"),
		TEXT("Mesh"),
		TEXT("Amplification"),
		TEXT("Pixel"),
		TEXT("Geometry"),
		TEXT("Compute"),
		TEXT("RayGen"),
		TEXT("RayMiss"),
		TEXT("RayHitGroup"),
		TEXT("RayCallable"),
		TEXT("WorkGraphRoot"),
		TEXT("WorkGraphComputeNode"),
	};

	/** Compile time check to verify that the GL mapping tables are up-to-date. */
	static_assert(SF_NumFrequencies == UE_ARRAY_COUNT(FrequencyStringTable), "NumFrequencies changed. Please update tables.");

	const TCHAR* GetFrequencyName(EShaderFrequency Frequency)
	{
		check((int32)Frequency >= 0 && Frequency < SF_NumFrequencies);
		return FrequencyStringTable[Frequency];
	}

	FHlslccHeader::FHlslccHeader() :
		Name(TEXT(""))
	{
		NumThreads[0] = NumThreads[1] = NumThreads[2] = 0;
	}

	bool FHlslccHeader::Read(const ANSICHAR*& ShaderSource, int32 SourceLen)
	{
#define DEF_PREFIX_STR(Str) \
		static const ANSICHAR* Str##Prefix = "// @" #Str ": "; \
		static const int32 Str##PrefixLen = FCStringAnsi::Strlen(Str##Prefix)
		DEF_PREFIX_STR(Inputs);
		DEF_PREFIX_STR(Outputs);
		DEF_PREFIX_STR(UniformBlocks);
		DEF_PREFIX_STR(Uniforms);
		DEF_PREFIX_STR(PackedGlobals);
		DEF_PREFIX_STR(PackedUB);
		DEF_PREFIX_STR(PackedUBCopies);
		DEF_PREFIX_STR(PackedUBGlobalCopies);
		DEF_PREFIX_STR(Samplers);
		DEF_PREFIX_STR(UAVs);
		DEF_PREFIX_STR(SamplerStates);
		DEF_PREFIX_STR(AccelerationStructures);
		DEF_PREFIX_STR(NumThreads);
#undef DEF_PREFIX_STR

		// Skip any comments that come before the signature.
		while (FCStringAnsi::Strncmp(ShaderSource, "//", 2) == 0 &&
			FCStringAnsi::Strncmp(ShaderSource + 2, " !", 2) != 0 &&
			FCStringAnsi::Strncmp(ShaderSource + 2, " @", 2) != 0)
		{
			ShaderSource += 2;
			while (*ShaderSource && *ShaderSource++ != '\n')
			{
				// Do nothing
			}
		}

		// Read shader name if any
		if (FCStringAnsi::Strncmp(ShaderSource, "// !", 4) == 0)
		{
			ShaderSource += 4;
			while (*ShaderSource && *ShaderSource != '\n')
			{
				Name += (TCHAR)*ShaderSource;
				++ShaderSource;
			}

			if (*ShaderSource == '\n')
			{
				++ShaderSource;
			}
		}

		// Skip any comments that come before the signature.
		while (FCStringAnsi::Strncmp(ShaderSource, "//", 2) == 0 &&
			FCStringAnsi::Strncmp(ShaderSource + 2, " @", 2) != 0)
		{
			ShaderSource += 2;
			while (*ShaderSource && *ShaderSource++ != '\n')
			{
				// Do nothing
			}
		}

		if (FCStringAnsi::Strncmp(ShaderSource, InputsPrefix, InputsPrefixLen) == 0)
		{
			ShaderSource += InputsPrefixLen;

			if (!ReadInOut(ShaderSource, Inputs))
			{
				return false;
			}
		}

		if (FCStringAnsi::Strncmp(ShaderSource, OutputsPrefix, OutputsPrefixLen) == 0)
		{
			ShaderSource += OutputsPrefixLen;

			if (!ReadInOut(ShaderSource, Outputs))
			{
				return false;
			}
		}

		if (FCStringAnsi::Strncmp(ShaderSource, UniformBlocksPrefix, UniformBlocksPrefixLen) == 0)
		{
			ShaderSource += UniformBlocksPrefixLen;

			while (*ShaderSource && *ShaderSource != '\n')
			{
				FAttribute UniformBlock;
				if (!ParseIdentifier(ShaderSource, UniformBlock.Name))
				{
					return false;
				}

				if (!Match(ShaderSource, '('))
				{
					return false;
				}
				
				if (!ParseIntegerNumber(ShaderSource, UniformBlock.Index))
				{
					return false;
				}

				if (!Match(ShaderSource, ')'))
				{
					return false;
				}

				UniformBlocks.Add(UniformBlock);

				if (Match(ShaderSource, '\n'))
				{
					break;
				}

				if (Match(ShaderSource, ','))
				{
					continue;
				}
			
				//#todo-rco: Need a log here
				//UE_LOGF(ShaderCompilerCommon, Warning, "Invalid char '%c'", *ShaderSource);
				return false;
			}
		}

		if (FCStringAnsi::Strncmp(ShaderSource, UniformsPrefix, UniformsPrefixLen) == 0)
		{
			// @todo-mobile: Will we ever need to support this code path?
			check(0);
			return false;
/*
			ShaderSource += UniformsPrefixLen;

			while (*ShaderSource && *ShaderSource != '\n')
			{
				uint16 ArrayIndex = 0;
				uint16 Offset = 0;
				uint16 NumComponents = 0;

				FString ParameterName = ParseIdentifier(ShaderSource);
				verify(ParameterName.Len() > 0);
				verify(Match(ShaderSource, '('));
				ArrayIndex = ParseNumber(ShaderSource);
				verify(Match(ShaderSource, ':'));
				Offset = ParseNumber(ShaderSource);
				verify(Match(ShaderSource, ':'));
				NumComponents = ParseNumber(ShaderSource);
				verify(Match(ShaderSource, ')'));

				ParameterMap.AddParameterAllocation(
					*ParameterName,
					ArrayIndex,
					Offset * BytesPerComponent,
					NumComponents * BytesPerComponent
					);

				if (ArrayIndex < OGL_NUM_PACKED_UNIFORM_ARRAYS)
				{
					PackedUniformSize[ArrayIndex] = FMath::Max<uint16>(
						PackedUniformSize[ArrayIndex],
						BytesPerComponent * (Offset + NumComponents)
						);
				}

				// Skip the comma.
				if (Match(ShaderSource, '\n'))
				{
					break;
				}

				verify(Match(ShaderSource, ','));
			}

			Match(ShaderSource, '\n');
*/
		}

		// @PackedGlobals: Global0(h:0,1),Global1(h:4,1),Global2(h:8,1)
		if (FCStringAnsi::Strncmp(ShaderSource, PackedGlobalsPrefix, PackedGlobalsPrefixLen) == 0)
		{
			ShaderSource += PackedGlobalsPrefixLen;
			while (*ShaderSource && *ShaderSource != '\n')
			{
				FPackedGlobal PackedGlobal;
				if (!ParseIdentifier(ShaderSource, PackedGlobal.Name))
				{
					return false;
				}

				if (!Match(ShaderSource, '('))
				{
					return false;
				}

				PackedGlobal.PackedType = *ShaderSource++;

				if (!Match(ShaderSource, ':'))
				{
					return false;
				}

				if (!ParseIntegerNumber(ShaderSource, PackedGlobal.Offset))
				{
					return false;
				}

				if (!Match(ShaderSource, ','))
				{
					return false;
				}

				if (!ParseIntegerNumber(ShaderSource, PackedGlobal.Count))
				{
					return false;
				}

				if (!Match(ShaderSource, ')'))
				{
					return false;
				}

				PackedGlobals.Add(PackedGlobal);

				// Break if EOL
				if (Match(ShaderSource, '\n'))
				{
					break;
				}

				// Has to be a comma!
				if (Match(ShaderSource, ','))
				{
					continue;
				}

				//#todo-rco: Need a log here
				//UE_LOGF(ShaderCompilerCommon, Warning, "Invalid char '%c'", *ShaderSource);
				return false;
			}
		}

		// Packed Uniform Buffers (Multiple lines)
		// @PackedUB: CBuffer(0): CBMember0(0,1),CBMember1(1,1)
		while (FCStringAnsi::Strncmp(ShaderSource, PackedUBPrefix, PackedUBPrefixLen) == 0)
		{
			ShaderSource += PackedUBPrefixLen;

			FPackedUB PackedUB;

			if (!ParseIdentifier(ShaderSource, PackedUB.Attribute.Name))
			{
				return false;
			}

			if (!Match(ShaderSource, '('))
			{
				return false;
			}
			
			if (!ParseIntegerNumber(ShaderSource, PackedUB.Attribute.Index))
			{
				return false;
			}

			if (!Match(ShaderSource, ')'))
			{
				return false;
			}

			if (!Match(ShaderSource, ':'))
			{
				return false;
			}

			if (!Match(ShaderSource, ' '))
			{
				return false;
			}

			while (*ShaderSource && *ShaderSource != '\n')
			{
				FPackedUB::FMember Member;
				ParseIdentifier(ShaderSource, Member.Name);
				if (!Match(ShaderSource, '('))
				{
					return false;
				}

				if (!ParseIntegerNumber(ShaderSource, Member.Offset))
				{
					return false;
				}
				
				if (!Match(ShaderSource, ','))
				{
					return false;
				}

				if (!ParseIntegerNumber(ShaderSource, Member.Count))
				{
					return false;
				}

				if (!Match(ShaderSource, ')'))
				{
					return false;
				}

				PackedUB.Members.Add(Member);

				// Break if EOL
				if (Match(ShaderSource, '\n'))
				{
					break;
				}

				// Has to be a comma!
				if (Match(ShaderSource, ','))
				{
					continue;
				}

				//#todo-rco: Need a log here
				//UE_LOGF(ShaderCompilerCommon, Warning, "Invalid char '%c'", *ShaderSource);
				return false;
			}

			PackedUBs.Add(PackedUB);
		}

		// @PackedUBCopies: 0:0-0:h:0:1,0:1-0:h:4:1,1:0-1:h:0:1
		if (FCStringAnsi::Strncmp(ShaderSource, PackedUBCopiesPrefix, PackedUBCopiesPrefixLen) == 0)
		{
			ShaderSource += PackedUBCopiesPrefixLen;
			if (!ReadCopies(ShaderSource, false, PackedUBCopies))
			{
				return false;
			}
		}

		// @PackedUBGlobalCopies: 0:0-h:12:1,0:1-h:16:1,1:0-h:20:1
		if (FCStringAnsi::Strncmp(ShaderSource, PackedUBGlobalCopiesPrefix, PackedUBGlobalCopiesPrefixLen) == 0)
		{
			ShaderSource += PackedUBGlobalCopiesPrefixLen;
			if (!ReadCopies(ShaderSource, true, PackedUBGlobalCopies))
			{
				return false;
			}
		}

		if (FCStringAnsi::Strncmp(ShaderSource, SamplersPrefix, SamplersPrefixLen) == 0)
		{
			ShaderSource += SamplersPrefixLen;

			while (*ShaderSource && *ShaderSource != '\n')
			{
				FSampler Sampler;

				if (!ParseIdentifier(ShaderSource, Sampler.Name))
				{
					return false;
				}

				if (!Match(ShaderSource, '('))
				{
					return false;
				}

				if (!ParseIntegerNumber(ShaderSource, Sampler.Offset))
				{
					return false;
				}

				if (!Match(ShaderSource, ':'))
				{
					return false;
				}

				if (!ParseIntegerNumber(ShaderSource, Sampler.Count))
				{
					return false;
				}

				if (Match(ShaderSource, '['))
				{
					// Sampler States
					do
					{
						FString SamplerState;
						
						if (!ParseIdentifier(ShaderSource, SamplerState))
						{
							return false;
						}

						Sampler.SamplerStates.Add(SamplerState);
					}
					while (Match(ShaderSource, ','));

					if (!Match(ShaderSource, ']'))
					{
						return false;
					}
				}

				if (!Match(ShaderSource, ')'))
				{
					return false;
				}

				Samplers.Add(Sampler);

				// Break if EOL
				if (Match(ShaderSource, '\n'))
				{
					break;
				}

				// Has to be a comma!
				if (Match(ShaderSource, ','))
				{
					continue;
				}

				//#todo-rco: Need a log here
				//UE_LOGF(ShaderCompilerCommon, Warning, "Invalid char '%c'", *ShaderSource);
				return false;
			}
		}

		if (FCStringAnsi::Strncmp(ShaderSource, UAVsPrefix, UAVsPrefixLen) == 0)
		{
			ShaderSource += UAVsPrefixLen;

			while (*ShaderSource && *ShaderSource != '\n')
			{
				FUAV UAV;

				if (!ParseIdentifier(ShaderSource, UAV.Name))
				{
					return false;
				}

				if (!Match(ShaderSource, '('))
				{
					return false;
				}

				if (!ParseIntegerNumber(ShaderSource, UAV.Offset))
				{
					return false;
				}

				if (!Match(ShaderSource, ':'))
				{
					return false;
				}

				if (!ParseIntegerNumber(ShaderSource, UAV.Count))
				{
					return false;
				}

				if (!Match(ShaderSource, ')'))
				{
					return false;
				}

				UAVs.Add(UAV);

				// Break if EOL
				if (Match(ShaderSource, '\n'))
				{
					break;
				}

				// Has to be a comma!
				if (Match(ShaderSource, ','))
				{
					continue;
				}

				//#todo-rco: Need a log here
				//UE_LOGF(ShaderCompilerCommon, Warning, "Invalid char '%c'", *ShaderSource);
				return false;
			}
		}

		if (FCStringAnsi::Strncmp(ShaderSource, SamplerStatesPrefix, SamplerStatesPrefixLen) == 0)
		{
			ShaderSource += SamplerStatesPrefixLen;
			while (*ShaderSource && *ShaderSource != '\n')
			{
				FAttribute SamplerState;
				if (!ParseIntegerNumber(ShaderSource, SamplerState.Index))
				{
					return false;
				}

				if (!Match(ShaderSource, ':'))
				{
					return false;
				}

				if (!ParseIdentifier(ShaderSource, SamplerState.Name))
				{
					return false;
				}

				SamplerStates.Add(SamplerState);

				// Break if EOL
				if (Match(ShaderSource, '\n'))
				{
					break;
				}

				// Has to be a comma!
				if (Match(ShaderSource, ','))
				{
					continue;
				}

				//#todo-rco: Need a log here
				//UE_LOGF(ShaderCompilerCommon, Warning, "Invalid char '%c'", *ShaderSource);
				return false;
			}
		}

		if (FCStringAnsi::Strncmp(ShaderSource, AccelerationStructuresPrefix, AccelerationStructuresPrefixLen) == 0)
		{
			ShaderSource += AccelerationStructuresPrefixLen;

			while (*ShaderSource && *ShaderSource != '\n')
			{
				FAccelerationStructure AccelerationStructure;

				if (!ParseIntegerNumber(ShaderSource, AccelerationStructure.Offset))
				{
					return false;
				}

				if (!Match(ShaderSource, ':'))
				{
					return false;
				}

				if (!ParseIdentifier(ShaderSource, AccelerationStructure.Name))
				{
					return false;
				}

				AccelerationStructures.Add(AccelerationStructure);

				if (Match(ShaderSource, '\n'))
				{
					break;
				}

				if (Match(ShaderSource, ','))
				{
					continue;
				}

				return false;
			}
		}

		if (FCStringAnsi::Strncmp(ShaderSource, NumThreadsPrefix, NumThreadsPrefixLen) == 0)
		{
			ShaderSource += NumThreadsPrefixLen;
			if (!ParseIntegerNumber(ShaderSource, NumThreads[0]))
			{
				return false;
			}
			if (!Match(ShaderSource, ','))
			{
				return false;
			}

			if (!Match(ShaderSource, ' '))
			{
				return false;
			}

			if (!ParseIntegerNumber(ShaderSource, NumThreads[1]))
			{
				return false;
			}

			if (!Match(ShaderSource, ','))
			{
				return false;
			}

			if (!Match(ShaderSource, ' '))
			{
				return false;
			}

			if (!ParseIntegerNumber(ShaderSource, NumThreads[2]))
			{
				return false;
			}

			if (!Match(ShaderSource, '\n'))
			{
				return false;
			}
		}
	
		return ParseCustomHeaderEntries(ShaderSource);
	}

	bool FHlslccHeader::ReadCopies(const ANSICHAR*& ShaderSource, bool bGlobals, TArray<FPackedUBCopy>& OutCopies)
	{
		while (*ShaderSource && *ShaderSource != '\n')
		{
			FPackedUBCopy PackedUBCopy;
			PackedUBCopy.DestUB = 0;

			if (!ParseIntegerNumber(ShaderSource, PackedUBCopy.SourceUB))
			{
				return false;
			}

			if (!Match(ShaderSource, ':'))
			{
				return false;
			}

			if (!ParseIntegerNumber(ShaderSource, PackedUBCopy.SourceOffset))
			{
				return false;
			}

			if (!Match(ShaderSource, '-'))
			{
				return false;
			}

			if (!bGlobals)
			{
				if (!ParseIntegerNumber(ShaderSource, PackedUBCopy.DestUB))
				{
					return false;
				}

				if (!Match(ShaderSource, ':'))
				{
					return false;
				}
			}

			PackedUBCopy.DestPackedType = *ShaderSource++;

			if (!Match(ShaderSource, ':'))
			{
				return false;
			}

			if (!ParseIntegerNumber(ShaderSource, PackedUBCopy.DestOffset))
			{
				return false;
			}

			if (!Match(ShaderSource, ':'))
			{
				return false;
			}

			if (!ParseIntegerNumber(ShaderSource, PackedUBCopy.Count))
			{
				return false;
			}

			OutCopies.Add(PackedUBCopy);

			// Break if EOL
			if (Match(ShaderSource, '\n'))
			{
				break;
			}

			// Has to be a comma!
			if (Match(ShaderSource, ','))
			{
				continue;
			}

			//#todo-rco: Need a log here
			//UE_LOGF(ShaderCompilerCommon, Warning, "Invalid char '%c'", *ShaderSource);
			return false;
		}

		return true;
	}

	bool FHlslccHeader::ReadInOut(const ANSICHAR*& ShaderSource, TArray<FInOut>& OutAttributes)
	{
		while (*ShaderSource && *ShaderSource != '\n')
		{
			FInOut Attribute;

			if (!ParseIdentifier(ShaderSource, Attribute.Type))
			{
				return false;
			}

			if (Match(ShaderSource, '['))
			{
				if (!ParseIntegerNumber(ShaderSource, Attribute.ArrayCount))
				{
					return false;
				}

				if (!Match(ShaderSource, ']'))
				{
					return false;
				}
			}
			else
			{
				Attribute.ArrayCount = 0;
			}

			if (Match(ShaderSource, ';'))
			{
				if (!ParseSignedNumber(ShaderSource, Attribute.Index))
				{
					return false;
				}
			}

			if (!Match(ShaderSource, ':'))
			{
				return false;
			}

			if (!ParseIdentifier(ShaderSource, Attribute.Name))
			{
				return false;
			}

			// Optional array suffix
			if (Match(ShaderSource, '['))
			{
				Attribute.Name += '[';
				while (*ShaderSource)
				{
					Attribute.Name += *ShaderSource;
					if (Match(ShaderSource, ']'))
					{
						break;
					}
					++ShaderSource;
				}
			}

			OutAttributes.Add(Attribute);

			// Break if EOL
			if (Match(ShaderSource, '\n'))
			{
				return true;
			}

			// Has to be a comma!
			if (Match(ShaderSource, ','))
			{
				continue;
			}

			//#todo-rco: Need a log here
			//UE_LOGF(ShaderCompilerCommon, Warning, "Invalid char '%c'", *ShaderSource);
			return false;
		}

		// Last character must be EOL
		return Match(ShaderSource, '\n');
	}

} // namespace CrossCompiler
