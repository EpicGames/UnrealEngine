// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderConductorContext.h"
#include "HAL/ExceptionHandling.h"
#include "ShaderCompilerDefinitions.h"
#include "Containers/AnsiString.h"

#include "ShaderCompilerCommonInternal.h"

#if PLATFORM_MAC || PLATFORM_WINDOWS || PLATFORM_LINUX
THIRD_PARTY_INCLUDES_START
	#include "ShaderConductor/ShaderConductor.hpp"
THIRD_PARTY_INCLUDES_END
#endif


namespace CrossCompiler
{

#if PLATFORM_MAC || PLATFORM_WINDOWS || PLATFORM_LINUX

	void FShaderConductorOptions::ApplyCommonCompilerFlags(const FShaderCompilerFlags& InCompilerFlags)
	{
		// Treat all warnings as errors
		bWarningsAsErrors = InCompilerFlags.Contains(CFLAG_WarningsAsErrors);

		// Include shader source code in the shader debug info. This allows source code correlation in shader profiling tools.
		bEnableDebugInfo = InCompilerFlags.Contains(CFLAG_ForceSpirvDebugInfo);

		// Enable HLSL 2021 if specified
		if (InCompilerFlags.Contains(CFLAG_HLSL2021))
		{
			HlslVersion = 2021;
		}
	}

	static void ScRewriteWrapper(
		const ShaderConductor::Compiler::SourceDesc& InSourceDesc,
		const ShaderConductor::Compiler::Options& InOptions,
		ShaderConductor::Compiler::ResultDesc& OutResultDesc)
	{
		OutResultDesc = ShaderConductor::Compiler::Rewrite(InSourceDesc, InOptions);
	}

	static void ScCompileWrapper(
		const ShaderConductor::Compiler::SourceDesc& InSourceDesc,
		const ShaderConductor::Compiler::Options& InOptions,
		const ShaderConductor::Compiler::TargetDesc& InTargetDesc,
		ShaderConductor::Compiler::ResultDesc& OutResultDesc)
	{
		OutResultDesc = ShaderConductor::Compiler::Compile(InSourceDesc, InOptions, InTargetDesc);
	}

	static void ScConvertBinaryWrapper(
		const ShaderConductor::Compiler::ResultDesc& InBinaryDesc,
		const ShaderConductor::Compiler::SourceDesc& InSourceDesc,
		const ShaderConductor::Compiler::TargetDesc& InTargetDesc,
		const ShaderConductor::Compiler::Options& InOptions,
		ShaderConductor::Compiler::ResultDesc& OutResultDesc)
	{
		OutResultDesc = ShaderConductor::Compiler::ConvertBinary(InBinaryDesc, InSourceDesc, InOptions, InTargetDesc);
	}

	// Converts the specified ShaderConductor blob to FString.
	static bool ConvertScBlobToFString(ShaderConductor::Blob* Blob, FString& OutString)
	{
		if (Blob && Blob->Size() > 0)
		{
			OutString = StringCast<TCHAR>(reinterpret_cast<const ANSICHAR*>(Blob->Data()), Blob->Size());
			return true;
		}
		return false;
	}

	static ShaderConductor::ShaderStage ToShaderConductorShaderStage(EShaderFrequency Frequency)
	{
		check(Frequency >= SF_Vertex && Frequency <= SF_RayCallable);
		switch (Frequency)
		{
		case SF_Vertex:			return ShaderConductor::ShaderStage::VertexShader;
		case SF_Mesh:			return ShaderConductor::ShaderStage::MeshShader;
        case SF_Amplification:	return ShaderConductor::ShaderStage::AmplificationShader;
		case SF_Pixel:			return ShaderConductor::ShaderStage::PixelShader;
		case SF_Geometry:		return ShaderConductor::ShaderStage::GeometryShader;
		case SF_Compute:		return ShaderConductor::ShaderStage::ComputeShader;

		case SF_RayGen:			return ShaderConductor::ShaderStage::RayGen;
		case SF_RayMiss:		return ShaderConductor::ShaderStage::RayMiss;
		case SF_RayHitGroup:	return ShaderConductor::ShaderStage::RayHitGroup;
		case SF_RayCallable:	return ShaderConductor::ShaderStage::RayCallable;

		default: break;
		}
		return ShaderConductor::ShaderStage::NumShaderStages;
	}

	struct FOptionalConvertedAnsiString
	{
		FOptionalConvertedAnsiString& operator=(FStringView WideView)
		{
			Storage.Empty();
			Storage.Append(WideView);
			AnsiView = FAnsiStringView(Storage);
			return *this;
		}

		FOptionalConvertedAnsiString& operator=(FAnsiStringView InAnsiView)
		{
			AnsiView = InAnsiView;
			return *this;
		}

		void CopyAnsi(FAnsiStringView InAnsiView)
		{
			Storage = InAnsiView;
			AnsiView = FAnsiStringView(Storage);
		}

		FAnsiString Storage;
		FAnsiStringView AnsiView;
	};

	// Wrapper structure to hold all intermediate buffers for ShaderConductor
	struct FShaderConductorContext::FShaderConductorIntermediates
	{
		FShaderConductorIntermediates()
			: Stage(ShaderConductor::ShaderStage::NumShaderStages)
		{
		}

		FOptionalConvertedAnsiString ShaderSource;
		FOptionalConvertedAnsiString Filename;
		FOptionalConvertedAnsiString EntryPoint;
		ShaderConductor::ShaderStage Stage;
		TArray<TPair<FAnsiString, FAnsiString>> Defines;
		TArray<ShaderConductor::MacroDefine> DefineRefs;
		TArray<TPair<FAnsiString, FAnsiString>> Flags;
		TArray<ShaderConductor::MacroDefine> FlagRefs;
		FAnsiString InternalDxcArgs;
		TArray<FAnsiString> CustomDxcArgs;
		TArray<ANSICHAR const*> CustomDxcArgRefs;
		TArray<ANSICHAR const*> DxcArgRefs;
	};

	static void ConvertScSourceDesc(const FShaderConductorContext::FShaderConductorIntermediates& Intermediates, ShaderConductor::Compiler::SourceDesc& OutSourceDesc)
	{
		// Convert descriptor with pointers to the ANSI strings
		OutSourceDesc.source = Intermediates.ShaderSource.AnsiView.GetData();
		OutSourceDesc.fileName = Intermediates.Filename.AnsiView.GetData();
		OutSourceDesc.entryPoint = Intermediates.EntryPoint.AnsiView.GetData();
		OutSourceDesc.stage = Intermediates.Stage;
		if (Intermediates.DefineRefs.Num() > 0)
		{
			OutSourceDesc.defines = Intermediates.DefineRefs.GetData();
			OutSourceDesc.numDefines = static_cast<uint32>(Intermediates.DefineRefs.Num());
		}
		else
		{
			OutSourceDesc.defines = nullptr;
			OutSourceDesc.numDefines = 0;
		}
	}

	static const ANSICHAR* GetHlslVersionString(int32 Version)
	{
		switch (Version)
		{
		case 50: return "50";
		case 60: return "60";
		case 61: return "61";
		case 62: return "62";
		case 63: return "63";
		case 64: return "64";
		case 65: return "65";
		case 66: return "66";
		default: return nullptr;
		}
	}

	static void ConvertScTargetDescLanguageHlsl(const FShaderConductorTarget& InTarget, ShaderConductor::Compiler::TargetDesc& OutTargetDesc)
	{
		OutTargetDesc.language = ShaderConductor::ShadingLanguage::Hlsl;
		OutTargetDesc.version = GetHlslVersionString(InTarget.Version);
		checkf(OutTargetDesc.version!=nullptr, TEXT("Unsupported target shader version for HLSL: SM%d.%d"), InTarget.Version / 10, InTarget.Version % 10);
	}

	static const ANSICHAR* GetGlslFamilyVersionString(int32 Version)
	{
		switch (Version)
		{
		//ESSL
		case 310: return "310";
		case 320: return "320";
		//GLSL
		case 330: return "330";
		case 430: return "430";
		case 440: return "440";
		case 450: return "450";
		case 460: return "460";
		default: return nullptr;
		}
	}

	static void ConvertScTargetDescLanguageGlslFamily(const FShaderConductorTarget& InTarget, ShaderConductor::Compiler::TargetDesc& OutTargetDesc)
	{
		OutTargetDesc.language = (InTarget.Language == EShaderConductorLanguage::Glsl ? ShaderConductor::ShadingLanguage::Glsl : ShaderConductor::ShadingLanguage::Essl);
		OutTargetDesc.version = GetGlslFamilyVersionString(InTarget.Version);
		checkf(OutTargetDesc.version!=nullptr, TEXT("Unsupported target shader version for GLSL family: %d"), InTarget.Version);
	}

	static const ANSICHAR* GetMetalFamilyVersionString(int32 Version)
	{
		switch (Version)
		{
		case 30100: return "30100";
        case 30000: return "30000";
		case 20400: return "20400";
		case 20300: return "20300";
		case 20200: return "20200";
		case 20100: return "20100";
		case 20000: return "20000";
		case 10200: return "10200";
		case 10100: return "10100";
		case 10000: return "10000";
		default: return nullptr;
		}
	}

	static void ConvertScTargetDescLanguageMetalFamily(const FShaderConductorTarget& InTarget, ShaderConductor::Compiler::TargetDesc& OutTargetDesc)
	{
		OutTargetDesc.language = (InTarget.Language == EShaderConductorLanguage::Metal_macOS ? ShaderConductor::ShadingLanguage::Msl_macOS : ShaderConductor::ShadingLanguage::Msl_iOS);
		OutTargetDesc.version = GetMetalFamilyVersionString(InTarget.Version);
		checkf(OutTargetDesc.version!=nullptr, TEXT("Unsupported target shader version for Metal family: %d"), InTarget.Version);
	}

	// Converts an array of FString to a C-style array of char* pointers
	static void ConvertStringArrayToAnsiArray(const TArray<FString>& InPairs, TArray<FAnsiString>& OutPairs, TArray<const char*>& OutPairRefs)
	{
		// Convert map into an array container
		TArray<ANSICHAR> Value;
		for (const FString& Iter : InPairs)
		{
			OutPairs.Emplace(Iter);
		}

		// Store references after all elements have been added to the container so the pointers remain valid
		OutPairRefs.SetNum(OutPairs.Num());
		for (int32 Index = 0; Index < OutPairs.Num(); ++Index)
		{
			OutPairRefs[Index] = *OutPairs[Index];
		}
	}

	// Converts a map of string pairs to a C-Style macro defines array
	static void ConvertDefineMapToMacroDefines(const FShaderCompilerDefinitions& Definitions, TArray<TPair<FAnsiString, FAnsiString>>& OutPairs, TArray<ShaderConductor::MacroDefine>& OutPairRefs)
	{
		// Convert map into an array container
		for (FShaderCompilerDefinitions::FConstIterator Iter(Definitions); Iter; ++Iter)
		{
			OutPairs.Emplace(Iter.Key(), Iter.Value());
		}

		// Store references after all elements have been added to the container so the pointers remain valid
		OutPairRefs.SetNum(OutPairs.Num());
		for (int32 Index = 0; Index < OutPairs.Num(); ++Index)
		{
			OutPairRefs[Index].name = *OutPairs[Index].Key;
			OutPairRefs[Index].value = *OutPairs[Index].Value;
		}
	}
	
	static void ConvertScTargetDesc(FShaderConductorContext::FShaderConductorIntermediates& Intermediates, const FShaderConductorTarget& InTarget, ShaderConductor::Compiler::TargetDesc& OutTargetDesc)
	{
		// Convert FString to ANSI string and store them as intermediates
		FMemory::Memzero(OutTargetDesc);

		Intermediates.Flags.Empty();
		Intermediates.FlagRefs.Empty();

		switch (InTarget.Language)
		{
		case EShaderConductorLanguage::Hlsl:
			ConvertScTargetDescLanguageHlsl(InTarget, OutTargetDesc);
			break;
		case EShaderConductorLanguage::Glsl:
		case EShaderConductorLanguage::Essl:
			ConvertScTargetDescLanguageGlslFamily(InTarget, OutTargetDesc);
			break;
		case EShaderConductorLanguage::Metal_macOS:
		case EShaderConductorLanguage::Metal_iOS:
			ConvertScTargetDescLanguageMetalFamily(InTarget, OutTargetDesc);
			break;
		}

		// Convert flags map into an array container
		ConvertDefineMapToMacroDefines(*InTarget.CompileFlags, Intermediates.Flags, Intermediates.FlagRefs);

		OutTargetDesc.options = Intermediates.FlagRefs.GetData();
		OutTargetDesc.numOptions = static_cast<uint32>(Intermediates.FlagRefs.Num());

		// Wrap input function into lambda to convert to ShaderConductor interface
		if (InTarget.VariableTypeRenameCallback)
		{
			OutTargetDesc.variableTypeRenameCallback = [InnerCallback = InTarget.VariableTypeRenameCallback](const char* VariableName, const char* TypeName) -> ShaderConductor::Blob
			{
				// Forward callback to public interface callback
				FString RenamedTypeName;
				if (InnerCallback(FAnsiStringView(VariableName), FAnsiStringView(TypeName), RenamedTypeName))
				{
					if (!RenamedTypeName.IsEmpty())
					{
						// Convert renamed type name from FString to ShaderConductor::Blob
						return ShaderConductor::Blob(TCHAR_TO_ANSI(*RenamedTypeName), RenamedTypeName.Len() + 1);
					}
				}
				return ShaderConductor::Blob{};
			};
		}
	}


	/*
	 * Reduced list of SPIRV-Tools optimization passes to avoid nested expressions in GLSL output.
	 * The order of passes has been adopted from the standard '-O' optimization configuration with the following passes removed:
	 * - LocalSingleBlockLoadStoreElimPass
	 * - LocalSingleStoreElimPass
	 * - LocalMultiStoreElimPass
	 * - SSARewritePass
	 */
	static const TCHAR* GSpirvTools_OptPasses_PresetRelaxNestedExpr = TEXT(
		"--wrap-opkill,"
		"--eliminate-dead-branches,"
		"--merge-return,"
		"--inline-entry-points-exhaustive,"
		"--eliminate-dead-functions,"
		"--eliminate-dead-code-aggressive,"
		"--private-to-local,"
		"--eliminate-dead-code-aggressive,"
		"--scalar-replacement,"
		"--convert-local-access-chains,"
		"--eliminate-dead-code-aggressive,"
		"--ccp,"
		"--eliminate-dead-code-aggressive,"
		"--loop-unroll,"
		"--eliminate-dead-branches,"
		"--redundancy-elimination,"
		"--combine-access-chains,"
		"--simplify-instructions,"
		"--scalar-replacement,"
		"--convert-local-access-chains,"
		"--eliminate-dead-code-aggressive,"
		"--vector-dce,"
		"--eliminate-dead-inserts,"
		"--eliminate-dead-branches,"
		"--simplify-instructions,"
		"--if-conversion,"
		"--copy-propagate-arrays,"
		"--reduce-load-size,"
		"--eliminate-dead-code-aggressive,"
		"--merge-blocks,"
		"--redundancy-elimination,"
		"--eliminate-dead-branches,"
		"--merge-blocks,"
		"--simplify-instructions,"
		"--eliminate-dead-members,"
		"--merge-blocks,"
		"--redundancy-elimination,"
		"--simplify-instructions,"
		"--eliminate-dead-code-aggressive,"
		"--cfg-cleanup"
	);

	static const TCHAR* SelectSpirvCustomOptimizationPasses(const FString& OptimizationPasses)
	{
		if (OptimizationPasses == TEXT("preset(relax-nested-expr)"))
		{
			return GSpirvTools_OptPasses_PresetRelaxNestedExpr;
		}
		else
		{
			// Interpret input argument as set of optimization passes
			return *OptimizationPasses;
		}
	}

	static void AppendDxcArguments(const FShaderConductorOptions& InOptions, TArray<const ANSICHAR*>& DxcArguments, bool bGenerateSpirv = true)
	{
		if(bGenerateSpirv)
		{
			DxcArguments.Add("-spirv");
		}
		
		DxcArguments.Add("-Qunused-arguments");

		switch (InOptions.HlslVersion)
		{
		case 2015:
			DxcArguments.Add("-HV");
			DxcArguments.Add("2015");
			break;
		case 2016:
			DxcArguments.Add("-HV");
			DxcArguments.Add("2016");
			break;
		case 2017:
			DxcArguments.Add("-HV");
			DxcArguments.Add("2017");
			break;
		case 2018:
			DxcArguments.Add("-HV");
			DxcArguments.Add("2018");
			break;
		case 2021:
			DxcArguments.Add("-HV");
			DxcArguments.Add("2021");
			break;
		default:
			checkf(false, TEXT("Invalid HLSL version: expected 2015, 2016, 2017, 2018, or 2021 but %u was specified"), InOptions.HlslVersion);
			break;
		}

		// Ignore unknwon attributes as UE uses custom attributes for intermediate source transformation
		DxcArguments.Add("-Wno-unknown-attributes");

		// We only treat warnings as errors for input source code, not intermediate source since DXC rewriter might produce new warnings the shader authors don't have control over.
		if (InOptions.bWarningsAsErrors)
		{
			DxcArguments.Add("-WX");
		}

		// Add additional DXC arguments that are not exposed by ShaderConductor API directly
		if (!InOptions.bDisableScalarBlockLayout)
		{
			DxcArguments.Add("-fvk-use-scalar-layout");
		}
		if (InOptions.bPreserveStorageInput)
		{
			DxcArguments.Add("-fspv-preserve-storage-input");
		}
		if (InOptions.bForceStorageImageFormat)
		{
			DxcArguments.Add("-fvk-force-storage-image-format");
		}
		if (InOptions.bSvPositionImplicitInvariant)
		{
			DxcArguments.Add("-fspv-svposition-implicit-invariant");
		}
		if (InOptions.bSupportPreciseOutputs)
		{
			DxcArguments.Add("-fspv-support-precise-outputs");
		}

		using ETargetEnvironment = CrossCompiler::FShaderConductorOptions::ETargetEnvironment;
		if (InOptions.bEnable16bitTypes)
		{
			DxcArguments.Add("-fspv-target-env=universal1.5");

			// ShaderConductor.cpp forgot to pipedown enable16bitTypes, so work arround by adding the parameter manually in here.
			DxcArguments.Add("-enable-16bit-types");
		}
		else
		{
			switch (InOptions.TargetEnvironment)
			{
			default:
				checkf(false, TEXT("Unexpected SPIR-V target environment: %d"), (uint32)InOptions.TargetEnvironment);
			case ETargetEnvironment::Vulkan_1_0:
				DxcArguments.Add("-fspv-target-env=vulkan1.0");
				break;
			case ETargetEnvironment::Vulkan_1_1:
				DxcArguments.Add("-fspv-target-env=vulkan1.1");
				break;
			case ETargetEnvironment::Vulkan_1_2:
				DxcArguments.Add("-fspv-target-env=vulkan1.2");
				break;
			case ETargetEnvironment::Vulkan_1_3:
				DxcArguments.Add("-fspv-target-env=vulkan1.3");
				break;
			}
		}
	}

	static void ConvertScOptions(FShaderConductorContext::FShaderConductorIntermediates& Intermediates, const FShaderConductorOptions& InOptions, ShaderConductor::Compiler::Options& OutOptions, bool bIgnoreCustomDxcArgs = false, bool bGenerateSpirv = true)
	{
		// Validate input shader model with respect to certain language features.
		checkf(
			(!InOptions.bEnable16bitTypes || InOptions.ShaderModel >= FHlslShaderModel{ 6, 2 }),
			TEXT("DXC option '-enable-16bit-types' only supported with SM6.2+ but SM%u.%u was specified"),
			InOptions.ShaderModel.Major, InOptions.ShaderModel.Minor
		);

		OutOptions.removeUnusedGlobals = InOptions.bRemoveUnusedGlobals;
		OutOptions.packMatricesInRowMajor = InOptions.bPackMatricesInRowMajor;
		OutOptions.enable16bitTypes = InOptions.bEnable16bitTypes;
		OutOptions.enableDebugInfo = InOptions.bEnableDebugInfo;
		OutOptions.disableOptimizations = InOptions.bDisableOptimizations;
		OutOptions.enableFMAPass = InOptions.bEnableFMAPass;
		OutOptions.enableSeparateSamplers = InOptions.bEnableSeparateSamplersInGlsl;
		OutOptions.shaderModel = ShaderConductor::Compiler::ShaderModel
		{
			static_cast<uint8>(InOptions.ShaderModel.Major),
			static_cast<uint8>(InOptions.ShaderModel.Minor)
		};

		TArray<ANSICHAR const*>& DxcArgRefs = Intermediates.DxcArgRefs;

		DxcArgRefs.Empty();

		AppendDxcArguments(InOptions, DxcArgRefs, bGenerateSpirv);

		if (!InOptions.SpirvCustomOptimizationPasses.IsEmpty())
		{
			Intermediates.InternalDxcArgs = FAnsiString::Printf("-Oconfig=%ls", SelectSpirvCustomOptimizationPasses(InOptions.SpirvCustomOptimizationPasses));
			DxcArgRefs.Add(*Intermediates.InternalDxcArgs);
		}

		if (DxcArgRefs.Num() > 0)
		{
			// Use DXC argument container and append custom arguments
			if (!bIgnoreCustomDxcArgs)
			{
				DxcArgRefs.Append(Intermediates.CustomDxcArgRefs);
			}
			OutOptions.numDXCArgs = DxcArgRefs.Num();
			OutOptions.DXCArgs = (const char**)DxcArgRefs.GetData();
		}
		else if (!bIgnoreCustomDxcArgs)
		{
			// Use custom DXC arguments only
			OutOptions.numDXCArgs = Intermediates.CustomDxcArgRefs.Num();
			OutOptions.DXCArgs = (const char**)Intermediates.CustomDxcArgRefs.GetData();
		}
		else
		{
			// No additional DXC arguments
			OutOptions.numDXCArgs = 0;
			OutOptions.DXCArgs = nullptr;
		}
	}

	// Returns whether the specified line of text contains only these characters, making it a valid line marker from DXC: ' ', '\t', '~', '^'
	static bool IsTextLineDxcLineMarker(const FString& Line)
	{
		bool bContainsLineMarkerChars = false;
		for (TCHAR Char : Line)
		{
			if (Char == TCHAR('~') || Char == TCHAR('^'))
			{
				// Line contains at least one of the necessary characters to be a potential DXC line marker.
				bContainsLineMarkerChars = true;
			}
			else if (!(Char == TCHAR(' ') || Char == TCHAR('\t')))
			{
				// Illegal character for a potential DXC line marker.
				return false;
			}
		}
		return bContainsLineMarkerChars;
	}

	// Converts the error blob from ShaderConductor into an array of error reports (of type FShaderCompilerError).
	static void ConvertScCompileErrors(ShaderConductor::Blob& ErrorBlob, TArray<FShaderCompilerError>& OutErrors)
	{
		FAnsiStringView ErrorString(static_cast<const ANSICHAR*>(ErrorBlob.Data()), ErrorBlob.Size());

		// Forward parsed array of lines to primary conversion function
		FShaderConductorContext::ConvertCompileErrors(ErrorString, OutErrors);
	}

	FShaderConductorTarget::FShaderConductorTarget()
	{
		CompileFlags = MakePimpl<FShaderCompilerDefinitions>();
	}

	FShaderConductorContext::FShaderConductorContext()
		: Intermediates(new FShaderConductorIntermediates())
	{
#if PLATFORM_WINDOWS
		ShaderConductor::Compiler::OverrideAllocator(UE::ShaderCompilerCommon::GetDxcMalloc());
#endif
	}

	FShaderConductorContext::~FShaderConductorContext()
	{
		delete Intermediates;
	}

	FShaderConductorContext::FShaderConductorContext(FShaderConductorContext&& Rhs)
		: Errors(MoveTemp(Rhs.Errors))
		, Intermediates(Rhs.Intermediates)
	{
		Rhs.Intermediates = nullptr;
	}

	FShaderConductorContext& FShaderConductorContext::operator = (FShaderConductorContext&& Rhs)
	{
		Errors = MoveTemp(Rhs.Errors);
		delete Intermediates;
		Intermediates = Rhs.Intermediates;
		Rhs.Intermediates = nullptr;
		return *this;
	}

	bool FShaderConductorContext::LoadSource(FStringView ShaderSource, const FString& Filename, const FString& EntryPoint, EShaderFrequency ShaderStage, const FShaderCompilerDefinitions* Definitions, const TArray<FString>* ExtraDxcArgs)
	{
		// Convert FString to ANSI string and store them as intermediates
		Intermediates->ShaderSource = ShaderSource;
		Intermediates->Filename = Filename;
		Intermediates->EntryPoint = EntryPoint;

		// Convert macro definitions map into an array container
		if (Definitions != nullptr)
		{
			ConvertDefineMapToMacroDefines(*Definitions, Intermediates->Defines, Intermediates->DefineRefs);
		}

		if (ExtraDxcArgs && ExtraDxcArgs->Num() > 0)
		{
			ConvertStringArrayToAnsiArray(*ExtraDxcArgs, Intermediates->CustomDxcArgs, Intermediates->CustomDxcArgRefs);
		}

		// Convert shader stage
		Intermediates->Stage = ToShaderConductorShaderStage(ShaderStage);

		return true;
	}

	bool FShaderConductorContext::LoadSource(FAnsiStringView ShaderSource, const FString& Filename, const FString& EntryPoint, EShaderFrequency ShaderStage, const FShaderCompilerDefinitions* Definitions, const TArray<FString>* ExtraDxcArgs)
	{
		Intermediates->ShaderSource = ShaderSource;
		Intermediates->Filename = Filename;
		Intermediates->EntryPoint = EntryPoint;

		// Convert macro definitions map into an array container
		if (Definitions != nullptr)
		{
			ConvertDefineMapToMacroDefines(*Definitions, Intermediates->Defines, Intermediates->DefineRefs);
		}

		if (ExtraDxcArgs && ExtraDxcArgs->Num() > 0)
		{
			ConvertStringArrayToAnsiArray(*ExtraDxcArgs, Intermediates->CustomDxcArgs, Intermediates->CustomDxcArgRefs);
		}

		// Convert shader stage
		Intermediates->Stage = ToShaderConductorShaderStage(ShaderStage);

		return true;
	}

	bool FShaderConductorContext::LoadSource(const ANSICHAR* ShaderSource, const ANSICHAR* Filename, const ANSICHAR* EntryPoint, EShaderFrequency ShaderStage, const FShaderCompilerDefinitions* Definitions, const TArray<FString>* ExtraDxcArgs)
	{
		Intermediates->ShaderSource = ShaderSource;
		Intermediates->Filename = Filename;
		Intermediates->EntryPoint = EntryPoint;

		// Convert macro definitions map into an array container
		if (Definitions != nullptr)
		{
			ConvertDefineMapToMacroDefines(*Definitions, Intermediates->Defines, Intermediates->DefineRefs);
		}

		if (ExtraDxcArgs && ExtraDxcArgs->Num() > 0)
		{
			ConvertStringArrayToAnsiArray(*ExtraDxcArgs, Intermediates->CustomDxcArgs, Intermediates->CustomDxcArgRefs);
		}

		// Convert shader stage
		Intermediates->Stage = ToShaderConductorShaderStage(ShaderStage);

		return true;
	}

    bool FShaderConductorContext::CompileHlslToDxil(const FShaderConductorOptions& Options, TArray<uint32>& OutDxil)
    {
		constexpr bool bGenerateSpirv = false;
		
        // Convert descriptors for ShaderConductor interface
        ShaderConductor::Compiler::SourceDesc ScSourceDesc;
        ConvertScSourceDesc(*Intermediates, ScSourceDesc);

        ShaderConductor::Compiler::TargetDesc ScTargetDesc;
        FMemory::Memzero(ScTargetDesc);
        ScTargetDesc.language = ShaderConductor::ShadingLanguage::Dxil;
		
        ShaderConductor::Compiler::Options ScOptions;
        ConvertScOptions(*Intermediates, Options, ScOptions, false, bGenerateSpirv);

        // Compile HLSL source code to DXIL
        bool bSucceeded = false;
        ShaderConductor::Compiler::ResultDesc ResultDesc;
        ScCompileWrapper(ScSourceDesc, ScOptions, ScTargetDesc, ResultDesc);

        if (!ResultDesc.hasError && ResultDesc.target.Size() > 0)
        {
            // Copy result blob into output DXIL module
            OutDxil = TArray<uint32>(reinterpret_cast<const uint32*>(ResultDesc.target.Data()), ResultDesc.target.Size() / 4);
            bSucceeded = true;
        }
        else
        {
            bSucceeded = false;
        }

        // Append compile error and warning to output reports
        ConvertScCompileErrors(ResultDesc.errorWarningMsg, Errors);

        return bSucceeded;
    }

	bool FShaderConductorContext::CompileHlslToSpirv(const FShaderConductorOptions& Options, TArray<uint32>& OutSpirv)
	{
		// Convert descriptors for ShaderConductor interface
		ShaderConductor::Compiler::SourceDesc ScSourceDesc;
		ConvertScSourceDesc(*Intermediates, ScSourceDesc);

		ShaderConductor::Compiler::TargetDesc ScTargetDesc;
		FMemory::Memzero(ScTargetDesc);
		ScTargetDesc.language = ShaderConductor::ShadingLanguage::SpirV;

		ShaderConductor::Compiler::Options ScOptions;
		ConvertScOptions(*Intermediates, Options, ScOptions);

		// Compile HLSL source code to SPIR-V
		bool bSucceeded = false;
		ShaderConductor::Compiler::ResultDesc ResultDesc;
		ScCompileWrapper(ScSourceDesc, ScOptions, ScTargetDesc, ResultDesc);

		if (!ResultDesc.hasError && ResultDesc.target.Size() > 0)
		{
			// Copy result blob into output SPIR-V module
			OutSpirv = TArray<uint32>(reinterpret_cast<const uint32*>(ResultDesc.target.Data()), ResultDesc.target.Size() / 4);
			bSucceeded = true;
		}
		
		// Append compile error and warning to output reports
		ConvertScCompileErrors(ResultDesc.errorWarningMsg, Errors);

		return bSucceeded;
	}

	void FShaderConductorContext::RemoveLineDirectives()
	{
		if (!Intermediates || GetSourceLength() == 0)
		{
			return;
		}

		Intermediates->ShaderSource.Storage.ReplaceInline("#line", "//ine");
	}

	bool FShaderConductorContext::OptimizeSpirv(TArray<uint32>& Spirv, const ANSICHAR* const* OptConfigs, int32 OptConfigCount)
	{
		// Ignore this call if no optimization configurations were specified
		if (OptConfigCount > 0)
		{
			check(OptConfigs != nullptr);

			// Convert input SPIR-V module to Blob instance for ShaderConductor interface
			ShaderConductor::Compiler::ResultDesc SpirvInput;
			SpirvInput.target = ShaderConductor::Blob(Spirv.GetData(), Spirv.Num() * sizeof(uint32));
			SpirvInput.isText = false;
			SpirvInput.hasError = false;

			// Run optimization passes through ShaderConductor
			ShaderConductor::Compiler::ResultDesc SpirvOutput = ShaderConductor::Compiler::Optimize(SpirvInput, OptConfigs, static_cast<uint32_t>(OptConfigCount));
			if (!SpirvOutput.hasError && SpirvOutput.target.Size() > 0)
			{
				// Convert Blob instance back to our SPIR-V module
				Spirv = TArray<uint32>(reinterpret_cast<const uint32*>(SpirvOutput.target.Data()), SpirvOutput.target.Size() / 4);
			}
			else
			{
				// Extract errors
				if (SpirvOutput.errorWarningMsg.Size() > 0)
				{
					FString ErrorString;
					if (ConvertScBlobToFString(&SpirvOutput.errorWarningMsg, ErrorString))
					{
						Errors.Add(*ErrorString);
					}
				}
				return false;
			}
		}
		return true;
	}

	bool FShaderConductorContext::CompileSpirvToSource(const FShaderConductorOptions& Options, const FShaderConductorTarget& Target, const void* InSpirv, uint32 InSpirvByteSize, FString& OutSource)
	{
		return CompileSpirvToSourceBuffer(
			Options, Target, InSpirv, InSpirvByteSize,
			[&OutSource](const void* Data, uint32 Size)
			{
				// Convert source buffer to FString
				FString Converted = FString::ConstructFromPtrSize(reinterpret_cast<const ANSICHAR*>(Data), Size);
				OutSource = MoveTemp(Converted);
			}
		);
	}

	bool FShaderConductorContext::CompileSpirvToSourceAnsi(const FShaderConductorOptions& Options, const FShaderConductorTarget& Target, const void* InSpirv, uint32 InSpirvByteSize, TArray<ANSICHAR>& OutSource)
	{
		return CompileSpirvToSourceBuffer(
			Options, Target, InSpirv, InSpirvByteSize,
			[&OutSource](const void* Data, uint32 Size)
			{
				// Convert source buffer to ANSI string
				FAnsiString Copy = FAnsiString::ConstructFromPtrSize(reinterpret_cast<const ANSICHAR*>(Data), Size);
				OutSource = MoveTemp(Copy.GetCharArray());
			}
		);
	}

	bool FShaderConductorContext::CompileSpirvToSourceBuffer(const FShaderConductorOptions& Options, const FShaderConductorTarget& Target, const void* InSpirv, uint32 InSpirvByteSize, const TFunction<void(const void* Data, uint32 Size)>& OutputCallback)
	{
		check(OutputCallback != nullptr);
		check(InSpirv != nullptr);
		check(InSpirvByteSize > 0);
		checkf(InSpirvByteSize % 4 == 0, TEXT("SPIR-V code unaligned. Size must be a multiple of 4, but %u was specified."), InSpirvByteSize);

		// Convert descriptors for ShaderConductor interface
		ShaderConductor::Compiler::SourceDesc ScSourceDesc;
		ConvertScSourceDesc(*Intermediates, ScSourceDesc);

		ShaderConductor::Compiler::TargetDesc ScTargetDesc;
		ConvertScTargetDesc(*Intermediates, Target, ScTargetDesc);

		ShaderConductor::Compiler::Options ScOptions;
		ConvertScOptions(*Intermediates, Options, ScOptions);

		ShaderConductor::Compiler::ResultDesc ScBinaryDesc;
		ScBinaryDesc.target.Reset(InSpirv, InSpirvByteSize);
		ScBinaryDesc.isText = false;
		ScBinaryDesc.hasError = false;

		// Convert the input SPIR-V into Metal high level source
		bool bSucceeded = false;
		ShaderConductor::Compiler::ResultDesc ResultDesc;
		ScConvertBinaryWrapper(ScBinaryDesc, ScSourceDesc, ScTargetDesc, ScOptions, ResultDesc);

		if (!ResultDesc.hasError && ResultDesc.target.Size() > 0)
		{
			// Copy result blob into output SPIR-V module
			OutputCallback(ResultDesc.target.Data(), ResultDesc.target.Size());
			bSucceeded = true;
		}
	
		// Append compile error and warning to output reports
		if (ResultDesc.errorWarningMsg.Size() > 0)
		{
			FString ErrorString;
			if (ConvertScBlobToFString(&ResultDesc.errorWarningMsg, ErrorString))
			{
				Errors.Add(*ErrorString);
			}
		}

		return bSucceeded;
	}

	void FShaderConductorContext::FlushErrors(TArray<FShaderCompilerError>& OutErrors)
	{
		if (OutErrors.Num() > 0)
		{
			// Append internal list of errors to output list, then clear internal list
			for (const FShaderCompilerError& ErrorEntry : Errors)
			{
				OutErrors.Add(ErrorEntry);
			}
			Errors.Empty();
		}
		else
		{
			// Move internal list of errors into output list
			OutErrors = MoveTemp(Errors);
		}
	}

	const ANSICHAR* FShaderConductorContext::GetSourceString() const
	{
		return (Intermediates->ShaderSource.AnsiView.Len() > 0 ? Intermediates->ShaderSource.AnsiView.GetData() : nullptr);
	}

	int32 FShaderConductorContext::GetSourceLength() const
	{
		return Intermediates->ShaderSource.AnsiView.Len();
	}

	static const TCHAR* GetHlslShaderModelProfile(ShaderConductor::ShaderStage Stage)
	{
		switch (Stage)
		{
		case ShaderConductor::ShaderStage::VertexShader:	return TEXT("vs");
		case ShaderConductor::ShaderStage::PixelShader:		return TEXT("ps");
		case ShaderConductor::ShaderStage::GeometryShader:	return TEXT("gs");
		case ShaderConductor::ShaderStage::HullShader:		return TEXT("hl");
		case ShaderConductor::ShaderStage::DomainShader:	return TEXT("ds");
		case ShaderConductor::ShaderStage::ComputeShader:	return TEXT("cs");
		default:											return TEXT("lib");
		}
	}

	FString FShaderConductorContext::GenerateDxcArguments(const FShaderConductorOptions& Options) const
	{
		FString CmdLineArgs = FString::Printf(
			TEXT("-E %hs -T %s_%d_%d"),
			Intermediates->EntryPoint.AnsiView.GetData(), GetHlslShaderModelProfile(Intermediates->Stage), (int32)Options.ShaderModel.Major, (int32)Options.ShaderModel.Minor
		);

		TArray<const ANSICHAR*> DxcArguments;
		AppendDxcArguments(Options, DxcArguments);

		for (const ANSICHAR* Argument : DxcArguments)
		{
			CmdLineArgs += TEXT(" ");
			CmdLineArgs += ANSI_TO_TCHAR(Argument);
		}

		return CmdLineArgs;
	}

	void FShaderConductorContext::ConvertCompileErrors(FAnsiStringView ErrorsString, TArray<FShaderCompilerError>& OutErrors)
	{
		auto ScanPastDigits = [](const ANSICHAR*& Ptr)
			{
				while (FChar::IsDigit(*(++Ptr))) {}
			};

		// if the null terminator character is included in the string view, remove it before parsing so whitespace trimming works correctly
		if (ErrorsString.EndsWith(0))
		{
			ErrorsString.LeftChopInline(1);
		}

		const ANSICHAR* ErrorBlobAnsiStart = ErrorsString.GetData();
		const ANSICHAR* ErrorBlobAnsiCur = ErrorBlobAnsiStart;
		const ANSICHAR* ErrorBlobAnsiEnd = ErrorBlobAnsiCur + ErrorsString.Len();
		const ANSICHAR* LastLineStart = ErrorBlobAnsiStart;
		const ANSICHAR* CurErrorStart = nullptr;
		int32 ColonCount = 0;

		// since DXC doesn't give us a good delimiter for error messages, we look for a line containing 3 colons with digits in between (which will be the case
		// for a file/line marker, i.e. /My/File/Shader.ush:269:29:) and use the start of that line as the start of a diagnostic message (and the end of the previous)
		while (ErrorBlobAnsiCur < ErrorBlobAnsiEnd)
		{
			if (*ErrorBlobAnsiCur == '\n')
			{
				LastLineStart = ErrorBlobAnsiCur + 1;
			}
			else if (*ErrorBlobAnsiCur == ':')
			{
				ScanPastDigits(ErrorBlobAnsiCur);
				if (*ErrorBlobAnsiCur++ == ':')
				{
					ScanPastDigits(ErrorBlobAnsiCur);
					if (*ErrorBlobAnsiCur++ == ':')
					{
						if (CurErrorStart != nullptr)
						{
							OutErrors.Add(FString(FAnsiStringView(CurErrorStart, LastLineStart - CurErrorStart).TrimStartAndEnd()));
						}
						CurErrorStart = LastLineStart;
					}
				}
			}
			ErrorBlobAnsiCur++;
		}
		// after the loop above ErrorBlobAnsiCur should be pointing to the end of the error blob, and CurErrorStart should point to the
		// start of the last error, so we need to add one more error containing the characters between the two
		if (CurErrorStart != nullptr)
		{
			OutErrors.Add(FString(FAnsiStringView(CurErrorStart, ErrorBlobAnsiEnd - CurErrorStart).TrimStartAndEnd()));
		}
	}

	bool FShaderConductorContext::Disassemble(EShaderConductorIR Language, const void* Binary, uint32 BinaryByteSize, TArray<ANSICHAR>& OutAssemblyText)
	{
		// Initialize Blob with input SPIR-V code
		ShaderConductor::Compiler::DisassembleDesc BinaryDesc;
		switch (Language)
		{
		case EShaderConductorIR::Spirv:
			BinaryDesc.language = ShaderConductor::ShadingLanguage::SpirV;
			break;
		case EShaderConductorIR::Dxil:
			BinaryDesc.language = ShaderConductor::ShadingLanguage::Dxil;
			break;
		}
		BinaryDesc.binary = reinterpret_cast<const uint8_t*>(Binary);
		BinaryDesc.binarySize = BinaryByteSize;

		// Disassemble via ShaderConductor interface
		ShaderConductor::Compiler::ResultDesc TextOutput = ShaderConductor::Compiler::Disassemble(BinaryDesc);
		if (TextOutput.isText && !TextOutput.hasError)
		{
			// Convert and return output to ANSI string
			FAnsiString Copy = FAnsiString::ConstructFromPtrSize(reinterpret_cast<const ANSICHAR*>(TextOutput.target.Data()), TextOutput.target.Size());
			OutAssemblyText = MoveTemp(Copy.GetCharArray());
			return true;
		}

		return false;
	}

#else // PLATFORM_MAC || PLATFORM_WINDOWS || PLATFORM_LINUX

	FShaderConductorContext::FShaderConductorContext()
	{
		checkf(0, TEXT("Cannot instantiate FShaderConductorContext for unsupported platform"));
	}

	FShaderConductorContext::~FShaderConductorContext()
	{
		// Dummy
	}

	FShaderConductorContext::FShaderConductorContext(FShaderConductorContext&& Rhs)
	{
		// Dummy
	}

	FShaderConductorContext& FShaderConductorContext::operator = (FShaderConductorContext&& Rhs)
	{
		return *this; // Dummy
	}

	bool FShaderConductorContext::LoadSource(const FString& ShaderSource, const FString& Filename, const FString& EntryPoint, EShaderFrequency ShaderStage, const FShaderCompilerDefinitions* Definitions, const TArray<FString>* ExtraDxcArgs)
	{
		return false; // Dummy
	}

	bool FShaderConductorContext::LoadSource(FStringView ShaderSource, const FString& Filename, const FString& EntryPoint, EShaderFrequency ShaderStage, const FShaderCompilerDefinitions* Definitions, const TArray<FString>* ExtraDxcArgs)
	{
		return false; // Dummy
	}

	bool FShaderConductorContext::LoadSource(const ANSICHAR* ShaderSource, const ANSICHAR* Filename, const ANSICHAR* EntryPoint, EShaderFrequency ShaderStage, const FShaderCompilerDefinitions* Definitions, const TArray<FString>* ExtraDxcArgs)
	{
		return false; // Dummy
	}

	bool FShaderConductorContext::RewriteHlsl(const FShaderConductorOptions& Options, FString* OutSource)
	{
		return false; // Dummy
	}
 
    bool FShaderConductorContext::CompileHlslToDxil(const FShaderConductorOptions& Options, TArray<uint32>& OutDxil)
    {
        return false; // Dummy
    }

	bool FShaderConductorContext::CompileHlslToSpirv(const FShaderConductorOptions& Options, TArray<uint32>& OutSpirv)
	{
		return false; // Dummy
	}

	bool FShaderConductorContext::CompileSpirvToSource(const FShaderConductorOptions& Options, const FShaderConductorTarget& Target, const void* InSpirv, uint32 InSpirvByteSize, FString& OutSource)
	{
		return false; // Dummy
	}

	bool FShaderConductorContext::CompileSpirvToSourceAnsi(const FShaderConductorOptions& Options, const FShaderConductorTarget& Target, const void* InSpirv, uint32 InSpirvByteSize, TArray<ANSICHAR>& OutSource)
	{
		return false; // Dummy
	}

	bool FShaderConductorContext::CompileSpirvToSourceBuffer(const FShaderConductorOptions& Options, const FShaderConductorTarget& Target, const void* InSpirv, uint32 InSpirvByteSize, const TFunction<void(const void* Data, uint32 Size)>& OutputCallback)
	{
		return false; // Dummy
	}

	void FShaderConductorContext::FlushErrors(TArray<FShaderCompilerError>& OutErrors)
	{
		// Dummy
	}

	const ANSICHAR* FShaderConductorContext::GetSourceString() const
	{
		return nullptr; // Dummy
	}

	int32 FShaderConductorContext::GetSourceLength() const
	{
		return 0; // Dummy
	}

	void FShaderConductorContext::ConvertCompileErrors(const TArray<FString>& ErrorStringLines, TArray<FShaderCompilerError>& OutErrors)
	{
		// Dummy
	}

	bool FShaderConductorContext::Disassemble(EShaderConductorIR Language, const void* Binary, uint32 BinaryByteSize, TArray<ANSICHAR>& OutAssemblyText)
	{
		return false; // Dummy
	}

#endif // PLATFORM_MAC || PLATFORM_WINDOWS || PLATFORM_LINUX

	bool FShaderConductorContext::IsIntermediateSpirvOutputVariable(const ANSICHAR* SpirvVariableName)
	{
		// This is only true for "temp.var.hullMainRetVal" which is generated by DXC as intermediate output variable to communicate patch constant data in a Hull Shader.
		return (SpirvVariableName != nullptr && FCStringAnsi::Strcmp(SpirvVariableName, FShaderConductorContext::GetIdentifierTable().IntermediateTessControlOutput) == 0);
	}

	const FShaderConductorIdentifierTable& FShaderConductorContext::GetIdentifierTable()
	{
		static const FShaderConductorIdentifierTable IdentifierTable
		{
			/*InputAttribute:*/					"in.var.ATTRIBUTE",
			/*GlobalsUniformBuffer:*/			"$Globals",
			/*IntermediateTessControlOutput:*/	"temp.var.hullMainRetVal",
			/*DummySampler:*/					"SPIRV_Cross_DummySampler",
		};
		return IdentifierTable;
	}

	static const TCHAR* GetGlslShaderFileExt(EShaderFrequency ShaderStage)
	{
		switch (ShaderStage)
		{
		case SF_Vertex:			return TEXT("vert");
		case SF_Mesh:			return TEXT("mesh");
		case SF_Amplification:	return TEXT("task");
		case SF_Pixel:			return TEXT("frag");
		case SF_Geometry:		return TEXT("geom");
		case SF_Compute:		return TEXT("comp");
		case SF_RayGen:			return TEXT("rgen");
		case SF_RayMiss:		return TEXT("rmiss");
		case SF_RayHitGroup:	return TEXT("rahit"); // rahit/rchit
		case SF_RayCallable:	return TEXT("rcall");
		default:				return TEXT("glsl");
		}
	}

	const TCHAR* FShaderConductorContext::GetShaderFileExt(EShaderConductorLanguage Language, EShaderFrequency ShaderStage)
	{
		switch (Language)
		{
		case EShaderConductorLanguage::Hlsl:		return TEXT("hlsl");
		case EShaderConductorLanguage::Glsl:		[[fallthrough]];
		case EShaderConductorLanguage::Essl:		return GetGlslShaderFileExt(ShaderStage);
		case EShaderConductorLanguage::Metal_macOS: [[fallthrough]];
		case EShaderConductorLanguage::Metal_iOS:	return TEXT("metal");
		default:									return TEXT("");
		}
	}

	static const TCHAR* LexToString(EShaderConductorIR Language)
	{
		switch (Language)
		{
		case EShaderConductorIR::Spirv: return TEXT("SPIR-V");
		case EShaderConductorIR::Dxil: return TEXT("DXIL");
		default: return TEXT("");
		}
	}

	bool FShaderConductorContext::Disassemble(EShaderConductorIR Language, const void* Binary, uint32 BinaryByteSize, FGenericShaderStat& OutShaderStat)
	{
		TArray<ANSICHAR> AssemblyText;
		if (FShaderConductorContext::Disassemble(Language, Binary, BinaryByteSize, AssemblyText))
		{
			OutShaderStat.StatName = LexToString(Language);
			OutShaderStat.Value.Set<FString>(ANSI_TO_TCHAR(AssemblyText.GetData()));
			OutShaderStat.Flags = FGenericShaderStat::EFlags::Hidden;
			OutShaderStat.TagName = FShaderStatTagNames::AnalysisArtifactsName;
			return true;
		}
		return false;
	}

	void FShaderConductorContext::Shutdown()
	{
#if PLATFORM_LINUX
		ShaderConductor::Compiler::Shutdown();
#endif
	}

} // namespace CrossCompiler
