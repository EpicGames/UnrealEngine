// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderMinifier.h"

#include "HAL/PlatformTime.h"
#include "Hash/xxhash.h"
#include "Logging/LogMacros.h"
#include "Tests/TestHarnessAdapter.h"
#include "String/Find.h"
#include "Algo/BinarySearch.h"
#include "Misc/MemStack.h"

#define UE_SHADER_MINIFIER_SSE (PLATFORM_CPU_X86_FAMILY && PLATFORM_ENABLE_VECTORINTRINSICS && PLATFORM_ALWAYS_HAS_SSE4_2)

#if UE_SHADER_MINIFIER_SSE
#include <emmintrin.h>
#include <nmmintrin.h>
#endif

DEFINE_LOG_CATEGORY_STATIC(LogShaderMinifier, Log, All);

// TODO:
// - preserve multi-line #define

namespace UE::ShaderMinifier
{

using FMemStackSetAllocator = TSetAllocator<TSparseArrayAllocator<TMemStackAllocator<>, TMemStackAllocator<>>, TMemStackAllocator<>>;
using FMemStackAllocator = TMemStackAllocator<>;

static FShaderSource::FViewType SubStrView(FShaderSource::FViewType S, int32 Start)
{
	Start = FMath::Min(Start, S.Len());
	int32 Len = S.Len() - Start;
	return FShaderSource::FViewType(S.GetData() + Start, Len);
}

static FShaderSource::FViewType SubStrView(FShaderSource::FViewType S, int32 Start, int32 Len)
{
	Start = FMath::Min(Start, S.Len());
	Len	  = FMath::Min(Len, S.Len() - Start);
	return FShaderSource::FViewType(S.GetData() + Start, Len);
}

template<typename TCondition>
static FShaderSource::FViewType SkipUntil(FShaderSource::FViewType Source, TCondition Cond)
{
	int32 Cursor = 0;
	const int32 SourceLen = Source.Len();
	while (Cursor < SourceLen)
	{
		if (Cond(FShaderSource::FViewType(Source.GetData() + Cursor, SourceLen - Cursor)))
		{
			break;
		}
		++Cursor;
	}
	return FShaderSource::FViewType(Source.GetData() + Cursor, SourceLen - Cursor);
}

static bool Equals(FShaderSource::FViewType A, FShaderSource::FViewType B)
{
	int32 Len = A.Len();
	if (Len != B.Len())
	{
		return false;
	}
	const FShaderSource::CharType* DataA = A.GetData();
	const FShaderSource::CharType* DataB = B.GetData();
	for (int32 I = 0; I < Len; ++I)
	{
		if (DataA[I] != DataB[I])
		{
			return false;
		}
	}
	return true;
}

static bool StartsWith(FShaderSource::FViewType Source, FShaderSource::FViewType Prefix)
{
	const int32 SourceLen = Source.Len();
	const int32 PrefixLen = Prefix.Len();
	if (PrefixLen > SourceLen)
	{
		return false;
	}
	FShaderSource::FViewType SourceView(Source.GetData(), PrefixLen);
	return Equals(SourceView, Prefix);
}

struct FCharacterFlags
{
	static constexpr uint8 None = 0;

	static constexpr uint8 Letter     = 1 << 0;
	static constexpr uint8 Number     = 1 << 1;
	static constexpr uint8 Underscore = 1 << 2;
	static constexpr uint8 Space      = 1 << 3;
	static constexpr uint8 Special    = 1 << 4;

	static constexpr uint8 PossibleIdentifierMask = Letter | Number | Underscore;

	FCharacterFlags()
	{
		for (char C = '0'; C <= '9'; ++C)
		{
			Flags[uint8(C)] |= Number;
		}

		for (char C = 'a'; C <= 'z'; ++C)
		{
			Flags[uint8(C)] |= Letter;
		}

		for (char C = 'A'; C <= 'Z'; ++C)
		{
			Flags[uint8(C)] |= Letter;
		}

		Flags[uint8('_')] |= (Underscore | Special);

		Flags[uint8(' ')]  |= Space;
		Flags[uint8('\f')] |= Space;
		Flags[uint8('\r')] |= Space;
		Flags[uint8('\n')] |= Space;
		Flags[uint8('\t')] |= Space;
		Flags[uint8('\v')] |= Space;

		Flags[uint8('\\')] |= Special;
		Flags[uint8('/')]  |= Special;
		Flags[uint8('#')]  |= Special;
		Flags[uint8('{')]  |= Special;
		Flags[uint8('}')]  |= Special;
		Flags[uint8('(')]  |= Special;
		Flags[uint8(')')]  |= Special;
	}

	bool IsSpace(FShaderSource::CharType C) const
	{
		return (Flags[uint8(C)] & Space) != 0;
	}

	bool IsNumber(FShaderSource::CharType C) const
	{
		return (Flags[uint8(C)] & Number) != 0;
	}

	bool IsPossibleIdentifierCharacter(FShaderSource::CharType C) const
	{
		return (Flags[uint8(C)] & PossibleIdentifierMask) != 0;
	}

	bool IsSpecial(FShaderSource::CharType C) const
	{
		return (Flags[uint8(C)] & Special) != 0;
	}

	uint8 Flags[256] = {};
};

static const FCharacterFlags GCharacterFlags;

static bool IsSpace(FShaderSource::CharType C)
{
	return GCharacterFlags.IsSpace(C);
}

static bool IsNumber(FShaderSource::CharType C)
{
	return GCharacterFlags.IsNumber(C);
}

static bool IsPossibleIdentifierCharacter(FShaderSource::CharType C)
{
	return GCharacterFlags.IsPossibleIdentifierCharacter(C);
}

static FShaderSource::FViewType ExtractOperator(FShaderSource::FViewType Source)
{
	FShaderSource::FViewType Result;

	if (Source.IsEmpty() || IsPossibleIdentifierCharacter(Source[0]))
	{
		return Result;
	}

	// NOTE: array is sorted by length to match complete operator character sequences first
	static const FShaderSource::FViewType SupportedOperators[] =
	{
		// three-character operators
		ANSITEXTVIEW("<<="),
		ANSITEXTVIEW(">>="),
		ANSITEXTVIEW("->*"),

		// two-character operators
		ANSITEXTVIEW("+="),
		ANSITEXTVIEW("++"),
		ANSITEXTVIEW("-="),
		ANSITEXTVIEW("--"),
		ANSITEXTVIEW("->"),
		ANSITEXTVIEW("*="),
		ANSITEXTVIEW("/="),
		ANSITEXTVIEW("%="),
		ANSITEXTVIEW("^="),
		ANSITEXTVIEW("&="),
		ANSITEXTVIEW("&&"),
		ANSITEXTVIEW("|="),
		ANSITEXTVIEW("||"),
		ANSITEXTVIEW("<<"),
		ANSITEXTVIEW("<="),
		ANSITEXTVIEW(">>"),
		ANSITEXTVIEW(">="),
		ANSITEXTVIEW("=="),
		ANSITEXTVIEW("!="),
		ANSITEXTVIEW("()"),
		ANSITEXTVIEW("[]"),

		// single character operators
		ANSITEXTVIEW("+"),
		ANSITEXTVIEW("-"),
		ANSITEXTVIEW("*"),
		ANSITEXTVIEW("/"),
		ANSITEXTVIEW("%"),
		ANSITEXTVIEW("^"),
		ANSITEXTVIEW("&"),
		ANSITEXTVIEW("|"),
		ANSITEXTVIEW("~"),
		ANSITEXTVIEW("!"),
		ANSITEXTVIEW("="),
		ANSITEXTVIEW("<"),
		ANSITEXTVIEW(">"),
	};

	for (FShaderSource::FViewType Operator : SupportedOperators)
	{
		if (StartsWith(Source, Operator))
		{
			Result = Source.SubStr(0, Operator.Len());
			break;
		}
	}

	return Result;
}

#if UE_SHADER_MINIFIER_SSE
template <int CompareType>
inline int32 ScanPastCharactersSimd(const FShaderSource::CharType* Source, int32 SourceLen, __m128i NeedleVec)
{
	constexpr int32 Mode = _SIDD_UBYTE_OPS | CompareType | _SIDD_MASKED_NEGATIVE_POLARITY;
	int32 Cursor = 0;
	while (Cursor < SourceLen)
	{
		__m128i Chunk = _mm_loadu_si128(reinterpret_cast<const __m128i*>(Source + Cursor));
		const int32 CompareResult = _mm_cmpistrc(NeedleVec, Chunk, Mode);
		if (CompareResult)
		{
			Cursor += _mm_cmpistri(NeedleVec, Chunk, Mode);
			break;
		}
		Cursor += 16;
	}
	return Cursor;
}
#endif // UE_SHADER_MINIFIER_SSE

static FShaderSource::FViewType SkipUntilNonIdentifierCharacter(FShaderSource::FViewType Source)
{
	const int32 SourceLen = Source.Len();
	int32 Cursor = 0;
	const FShaderSource::CharType* SourceData = Source.GetData();

#if UE_SHADER_MINIFIER_SSE
	const __m128i NeedleVec = _mm_setr_epi8('0', '9', 'a', 'z', 'A', 'Z', '_', '_', 0, 0, 0, 0, 0, 0, 0, 0);

	Cursor = ScanPastCharactersSimd<_SIDD_CMP_RANGES>(SourceData, SourceLen, NeedleVec);
#else
	while (Cursor < SourceLen)
	{
		if (!IsPossibleIdentifierCharacter(SourceData[Cursor]))
		{
			break;
		}
		++Cursor;
	}
#endif // UE_SHADER_MINIFIER_SSE
	return FShaderSource::FViewType(SourceData + Cursor, FMath::Max(SourceLen - Cursor, 0));
}

static FShaderSource::FViewType SkipUntilNonNumber(FShaderSource::FViewType Source)
{
	const int32 SourceLen = Source.Len();
	int32 Cursor = 0;
	const FShaderSource::CharType* SourceData = Source.GetData();
	while (Cursor < SourceLen)
	{
		if (!IsNumber(SourceData[Cursor]))
		{
			break;
		}
		++Cursor;
	}
	return FShaderSource::FViewType(SourceData + Cursor, SourceLen - Cursor);
}

static FShaderSource::FViewType SkipSpace(FShaderSource::FViewType Source)
{
	const int32 SourceLen = Source.Len();
	int32 Cursor = 0;
	const FShaderSource::CharType* SourceData = Source.GetData();

#if UE_SHADER_MINIFIER_SSE
	const __m128i NeedleVec = _mm_setr_epi8(' ', '\f', '\r', '\n', '\t', '\v', 0, 0, 0, 0, 0, 0, 0, 0, 0, 0);

	Cursor = ScanPastCharactersSimd<_SIDD_CMP_EQUAL_ANY>(SourceData, SourceLen, NeedleVec);
#else
	while (Cursor < SourceLen)
	{
		if (!IsSpace(SourceData[Cursor]))
		{
			break;
		}
		++Cursor;
	}
#endif // UE_SHADER_MINIFIER_SSE
	return FShaderSource::FViewType(SourceData + Cursor, FMath::Max(SourceLen - Cursor, 0));
}

static FShaderSource::FViewType TrimSpace(FShaderSource::FViewType Source)
{
	int32 CursorBegin = 0;
	int32 CursorEnd = Source.Len();

	while (CursorBegin != CursorEnd)
	{
		if (!IsSpace(Source[CursorBegin]))
		{
			break;
		}
		++CursorBegin;
	}

	while (CursorBegin != CursorEnd)
	{
		if (!IsSpace(Source[CursorEnd-1]))
		{
			break;
		}
		--CursorEnd;
	}

	FShaderSource::FViewType Result = SubStrView(Source, CursorBegin, CursorEnd-CursorBegin);

	return Result;
}

static FShaderSource::FViewType SkipUntilNextLine(FShaderSource::FViewType Source)
{
	int32 Index = INDEX_NONE;
	if (Source.FindChar('\n', Index))
	{
		return FShaderSource::FViewType(Source.GetData() + Index, Source.Len() - Index);
	}
	else
	{
		return FShaderSource::FViewType {};
	}
}

static FShaderSource::FViewType SkipUntilStr(FShaderSource::FViewType Haystack, FShaderSource::FViewType Needle)
{
	return SkipUntil(Haystack, [Needle](FShaderSource::FViewType  S) { return StartsWith(S, Needle); });
}

static FShaderSource::FViewType ExtractBlock(FShaderSource::FViewType Source, FShaderSource::CharType DelimBegin, FShaderSource::CharType DelimEnd, TArray<FShaderSource::FViewType>& OutLineDirectives)
{
	// TODO: handle comments
	// TODO: handle #if 0 blocks

	int32 PosEnd = INDEX_NONE;
	int32 Stack  = 0;
	const int32 SourceLen = Source.Len();
	const FShaderSource::CharType* SourceData = Source.GetData();

	int32 Cursor = 0;

	enum class EStatus
	{
		Finished,
		Continue,
	};

	EStatus Status = EStatus::Continue;

	auto ProcessCharacter = [&Stack, &PosEnd, &OutLineDirectives, SourceData, SourceLen, DelimBegin, DelimEnd](int32 Cursor) -> EStatus
	{
		FShaderSource::CharType C = SourceData[Cursor];

		if (C == DelimBegin)
		{
			Stack++;
		}
		else if (C == DelimEnd)
		{
			if (Stack == 0)
			{
				// delimiter mismatch
				return EStatus::Finished;
			}

			Stack--;

			if (Stack == 0)
			{
				PosEnd = Cursor;
				return EStatus::Finished;
			}
		}
		else if (C == '#')
		{
			FShaderSource::FViewType Source(SourceData + Cursor, SourceLen - Cursor);
			if (StartsWith(Source, ANSITEXTVIEW("#line")))
			{
				FShaderSource::FViewType Remainder = SkipUntilNextLine(Source);
				FShaderSource::FViewType Block = SubStrView(Source, 0, Source.Len() - Remainder.Len());
				OutLineDirectives.Add(Block);
			}
		}

		return EStatus::Continue;
	};

#if UE_SHADER_MINIFIER_SSE
	const __m128i NeedleVec = _mm_setr_epi8(DelimBegin, DelimEnd, '#', 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0);

	while (Cursor < SourceLen && Status != EStatus::Finished)
	{
		__m128i Chunk = _mm_loadu_si128(reinterpret_cast<const __m128i*>(SourceData + Cursor));
		constexpr int32 Mode = _SIDD_UBYTE_OPS | _SIDD_CMP_EQUAL_ANY | _SIDD_MOST_SIGNIFICANT;
		const int32 CompareResult = _mm_cmpistrc(NeedleVec, Chunk, Mode);
		if (CompareResult)
		{
			__m128i MaskVec = _mm_cmpistrm(NeedleVec, Chunk, Mode);
			uint32 Mask = _mm_movemask_epi8(MaskVec);
			while (Mask != 0 && Status != EStatus::Finished)
			{
				const uint32 BitIndex = FMath::CountTrailingZeros(Mask);
				const uint32 ChunkCharIndex = BitIndex / sizeof(FShaderSource::CharType);
				Status = ProcessCharacter(Cursor + ChunkCharIndex);
				Mask &= ~(1u << BitIndex);
			}
		}
		Cursor += 16;
	}
#else
	while (Cursor < SourceLen && Status != EStatus::Finished)
	{
		Status = ProcessCharacter(Cursor);
		++Cursor;
	}
#endif // UE_SHADER_MINIFIER_SSE
	if (Stack == 0 && PosEnd != INDEX_NONE)
	{
		return FShaderSource::FViewType(Source.GetData(), PosEnd + 1);
	}
	else
	{
		return FShaderSource::FViewType{};
	}
}

enum class EBlockType : uint8 {
	Unknown,	// various identifiers and keywords that we did not need to or could not identify
	Keyword,	// e.g. struct, switch, register
	Attribute,	// e.g. `[numthreads(8,8,1)]`
	Type,		// return type of function or struct/cbuffer/variable type
	Base,		// inheritance base type
	Name,		// struct/variable/function name
	Binding,	// e.g. `register(t0, space1)` or `SV_Target0`
	Args,
	Body,
	Subscript,
	TemplateArgs,
	Expression,
	Directive,  // #define, #pragma, #line, etc.
	NamespaceDelimiter, // e.g. :: in an identifier like Foo::bar
	PtrOrRef, // e.g. '*' or '&' as part of the type
	OperatorName, // overladed operator, e.g. '+', '+=', etc.
	NodeId, // e.g. [NodeID(Foo, 0)]
};

struct FCodeBlock
{
	const FShaderSource::CharType* CodePtr = nullptr;
	int32        CodeLen = 0;
	EBlockType   Type = EBlockType::Unknown;
	uint8        Padding[3] = {};

	operator FShaderSource::FViewType () const
	{
		return GetCode();
	}

	bool operator == (const FShaderSource::FViewType S) const
	{
		return Equals(GetCode(), S);
	}

	void SetCode(FShaderSource::FViewType Code)
	{
		CodePtr = Code.GetData();
		CodeLen = Code.Len();
	}

	FShaderSource::FViewType GetCode() const
	{
		return FShaderSource::FViewType(CodePtr, CodeLen);
	}
};

static_assert(sizeof(FCodeBlock) == 16, "Unexpected FCodeBlock size");

enum class ECodeChunkType {
	Unknown,
	Struct,
	CBuffer,  // HLSL cbuffer block possibly without trailing ';'
	Function,
	Operator,
	Variable,
	Enum,
	Define,
	Pragma,
	CommentLine, // Single line comment
	Namespace,
	Using,
	Typedef,
};

struct FNamespace
{
	FNamespace() = default;
	FNamespace(TConstArrayView<FShaderSource::FViewType> InStack)
	{
		if (!InStack.IsEmpty())
		{
			for (const FShaderSource::FViewType& Part : InStack)
			{
				FullName += Part;
				FullName += "::";
			}
			FullName.LeftChopInline(2);
		}
		Stack = InStack;
	}

	FString FullName; // i.e. Foo::Bar::Baz
	TArray<FShaderSource::FViewType> Stack; // i.e. [Foo, Bar, Baz]
};

using FCodeBlockArray = TArray<FCodeBlock, TInlineAllocator<6>>;

struct FCodeChunk
{
	ECodeChunkType Type = ECodeChunkType::Unknown;
	FCodeBlockArray Blocks;

	int32 Namespace = INDEX_NONE; // Unique namespace ID (INDEX_NONE = global)

	// Indicates whether the code for this chunk can be used as-is.
	// One example where we have to do custom code emission is when a named struct and a variable are declared in one chunk.
	// The struct type may be referenced, but the variable may be removed. In this case we have to emit the type declaration only.
	bool bVerbatim = true;

	FShaderSource::FViewType FindFirstBlockByType(EBlockType InType) const
	{
		for (const FCodeBlock& Block : Blocks)
		{
			if (Block.Type == InType)
			{
				return Block;
			}
		}
		return {};
	}

	// String view covering the entire code chunk
	FShaderSource::FViewType GetCode() const
	{
		if (Blocks.IsEmpty())
		{
			return {};
		}
		else
		{
			const FCodeBlock& FirstBlock = Blocks[0];
			const FCodeBlock& LastBlock = Blocks[Blocks.Num()-1];
			const FShaderSource::CharType* Begin = FirstBlock.CodePtr;
			const FShaderSource::CharType* End = LastBlock.CodePtr + LastBlock.CodeLen;
			return FShaderSource::FViewType(Begin, int32(End-Begin));
		}
	}
};

struct FParsedShader
{
	FShaderSource::FViewType Source;
	TArray<FCodeChunk> Chunks;
	TArray<FNamespace> Namespaces;
	TArray<FShaderSource::FViewType> LineDirectives;
};

struct FNamespaceTracker
{
	TMap<FString, int32> UniqueNamespaceMap;
	TArray<FNamespace> UniqueNamespaceArray;
	TArray<FShaderSource::FViewType> NamespaceStack;
	TArray<int32> NamespaceIdStack;

	FNamespaceTracker() = default;

	void Push(FShaderSource::FViewType Name)
	{
		NamespaceStack.Push(Name);
		FNamespace NamespaceEntry(NamespaceStack);
		int32& EntryIndex = UniqueNamespaceMap.FindOrAdd(NamespaceEntry.FullName, INDEX_NONE);
		if (EntryIndex == INDEX_NONE)
		{
			EntryIndex = UniqueNamespaceArray.Num();
			UniqueNamespaceArray.Add(MoveTemp(NamespaceEntry));
		}
		NamespaceIdStack.Push(EntryIndex);
	}

	bool Pop()
	{
		if (NamespaceStack.IsEmpty())
		{
			return false;
		}
		else
		{
			NamespaceStack.Pop();
			NamespaceIdStack.Pop();
			return true;
		}
	}

	int32 CurrentId() const
	{
		return NamespaceIdStack.IsEmpty() ? INDEX_NONE : NamespaceIdStack.Last();
	}
};

FShaderSource::FViewType ExtractNextIdentifier(FShaderSource::FViewType Source)
{
	FShaderSource::FViewType Remainder = SkipUntilNonIdentifierCharacter(Source);
	FShaderSource::FViewType Identifier = SubStrView(Source, 0, Source.Len() - Remainder.Len());
	return Identifier;
}

static FParsedShader ParseShader(const FShaderSource& InSource, FDiagnostics& Output)
{
	FParsedShader Result;

	Result.Source = InSource.GetView();

	FShaderSource::FViewType	Source = InSource.GetView();
	FCodeBlockArray			PendingBlocks;
	TArray<FCodeChunk>		Chunks;

	ECodeChunkType ChunkType = ECodeChunkType::Unknown;
	bool		   bFoundBody = false;
	bool		   bFoundColon = false;
	bool		   bFoundIdentifier = false;
	bool		   bFoundAssignment = false;

	int32 ArgsBlockIndex = INDEX_NONE;
	int32 CbufferBlockIndex = INDEX_NONE;
	int32 StructBlockIndex = INDEX_NONE;
	int32 EnumBlockIndex = INDEX_NONE;
	int32 BodyBlockIndex = INDEX_NONE;
	int32 ExpressionBlockIndex = INDEX_NONE;
	int32 OperatorKeywordBlockIndex = INDEX_NONE;

	FNamespaceTracker NamespaceTracker;
	FShaderSource::FViewType PendingNamespace;

	auto AddDiagnostic = [InSource, &Source](TArray<FDiagnosticMessage>& Output, FStringView Message)
	{
		FDiagnosticMessage Diagnostic;

		Diagnostic.Message = FString(Message);
		Diagnostic.Offset = int32(Source.GetData() - InSource.GetView().GetData());
		// Diagnostic.Line = ...; // TODO
		// Diagnostic.Column = ...; // TODO

		Output.Add(MoveTemp(Diagnostic));
	};

	auto AddBlock = [&PendingBlocks](EBlockType Type, FShaderSource::FViewType Code)
	{
		FCodeBlock NewBlock;
		NewBlock.Type = Type;
		NewBlock.SetCode(Code);
		PendingBlocks.Push(NewBlock);
	};

	auto FinalizeChunk = [&]()
	{
		const bool bFoundArgs = ArgsBlockIndex >= 0;

		bool bHasType = false;
		bool bHasName = false;

		if (!PendingBlocks.IsEmpty())
		{
			if (ChunkType == ECodeChunkType::Unknown)
			{
				if (bFoundIdentifier && bFoundArgs && bFoundBody)
				{
					ChunkType = ECodeChunkType::Function;
				}
				else if (bFoundIdentifier)
				{
					ChunkType = ECodeChunkType::Variable;
				}
			}

			int32 NameBlockIndex = INDEX_NONE;

			if (ChunkType == ECodeChunkType::Struct)
			{
				check(StructBlockIndex >= 0);

				PendingBlocks[StructBlockIndex].Type = EBlockType::Keyword;

				int32 TypeBlockIndex = StructBlockIndex + 1;
				if (TypeBlockIndex != BodyBlockIndex && TypeBlockIndex < PendingBlocks.Num())
				{
					PendingBlocks[TypeBlockIndex].Type = EBlockType::Type;
					bHasType = true;
				}

				// If struct body is not the last block, it must be followed by a variable name
				// i.e. `struct Foo { ... } Blah;` or `struct { ... } Blah;` or `struct Foo { ... } Blah = { expression };`
				if (BodyBlockIndex > 0 && BodyBlockIndex + 1 < PendingBlocks.Num())
				{
					NameBlockIndex = BodyBlockIndex + 1;
					PendingBlocks[NameBlockIndex].Type = EBlockType::Name;
					bHasName = true;
				}

				// If there is an expression block, we expect a named variable to also exist
				// i.e. `struct Foo { ... } Blah = { expression };` 
				if (ExpressionBlockIndex > 0 && NameBlockIndex == INDEX_NONE)
				{
					AddDiagnostic(Output.Errors, TEXTVIEW("Initialized struct variables must be named"));
					return;
				}
			}
			else if (ChunkType == ECodeChunkType::CBuffer)
			{
				check(CbufferBlockIndex >= 0);

				PendingBlocks[CbufferBlockIndex].Type = EBlockType::Keyword;

				int32 TypeBlockIndex = CbufferBlockIndex + 1;
				if (TypeBlockIndex != BodyBlockIndex && TypeBlockIndex < PendingBlocks.Num())
				{
					PendingBlocks[TypeBlockIndex].Type = EBlockType::Type;
				}
			}
			else if (ChunkType == ECodeChunkType::Enum)
			{
				check(EnumBlockIndex >= 0);

				PendingBlocks[EnumBlockIndex].Type = EBlockType::Keyword;

				if (BodyBlockIndex > 1)
				{
					PendingBlocks[BodyBlockIndex - 1].Type = EBlockType::Type;
				}
			}
			else if (ChunkType == ECodeChunkType::Function)
			{
				NameBlockIndex = ArgsBlockIndex - 1;
				if (NameBlockIndex >= 0)
				{
					PendingBlocks[NameBlockIndex].Type = EBlockType::Name;
				}
			}
			else if (ChunkType == ECodeChunkType::Variable)
			{
				// TODO: tag name / type / binding
			}
			else if (ChunkType == ECodeChunkType::Typedef)
			{
				NameBlockIndex = PendingBlocks.Num() - 1;
				for (int32 Index = 1; Index < NameBlockIndex; Index++)
				{
					PendingBlocks[Index].Type = EBlockType::Type;
				}
				PendingBlocks[NameBlockIndex].Type = EBlockType::Name;
			}
			else if (ChunkType == ECodeChunkType::Operator)
			{
				// Treat any uncategorized blocks as part of the type
				for (int32 Index = 0; Index < OperatorKeywordBlockIndex; Index++)
				{
					if (PendingBlocks[Index].Type == EBlockType::Unknown)
					{
						PendingBlocks[Index].Type = EBlockType::Type;
					}
				}
			}

			if (ChunkType == ECodeChunkType::Struct && bHasName && !bHasType)
			{
				ChunkType = ECodeChunkType::Variable;
			}

			const int32 Namespace = NamespaceTracker.CurrentId();

			if (ChunkType == ECodeChunkType::Struct && bHasName && bHasType)
			{
				// Handle simultaneous struct type and variable declaration

				FCodeChunk StructChunk;
				StructChunk.Type = ECodeChunkType::Struct;
				StructChunk.bVerbatim = false;

				for (int32 i = int32(StructBlockIndex); i < NameBlockIndex; ++i)
				{
					StructChunk.Blocks.Push(PendingBlocks[i]);
				}

				FCodeChunk VarChunk;
				VarChunk.Type = ECodeChunkType::Variable;
				VarChunk.bVerbatim = false;

				for (int32 i = 0; i < PendingBlocks.Num(); ++i)
				{
					if (i == StructBlockIndex || i == BodyBlockIndex)
					{
						continue;
					}
					VarChunk.Blocks.Push(PendingBlocks[i]);
				}

				StructChunk.Namespace = Namespace;
				VarChunk.Namespace = Namespace;

				Chunks.Push(StructChunk);
				Chunks.Push(VarChunk);
			}
			else
			{
				FCodeChunk Chunk;
				Chunk.Type = ChunkType;
				Chunk.Namespace = Namespace;
				Swap(Chunk.Blocks, PendingBlocks);
				Chunks.Push(Chunk);
			}

			ChunkType = ECodeChunkType::Unknown;
			ArgsBlockIndex = INDEX_NONE;
			CbufferBlockIndex = INDEX_NONE;
			StructBlockIndex = INDEX_NONE;
			EnumBlockIndex = INDEX_NONE;
			BodyBlockIndex = INDEX_NONE;
			ExpressionBlockIndex = INDEX_NONE;
			OperatorKeywordBlockIndex = INDEX_NONE;
			bFoundBody = false;
			bFoundColon = false;
			bFoundIdentifier = false;
			bFoundAssignment = false;

			PendingBlocks.Reset();
		}
	};

	while (Output.Errors.IsEmpty())
	{
		Source = SkipSpace(Source);

		if (Source.IsEmpty())
		{
			break;
		}

		const FShaderSource::CharType FirstChar = *Source.GetData();

		if (GCharacterFlags.IsSpecial(FirstChar))
		{
			if (FirstChar == '/')
			{
				if (StartsWith(Source, ANSITEXTVIEW("//")))
				{
					FShaderSource::FViewType Remainder = SkipUntilNextLine(Source);

					// Save comment lines that are outside of blocks
					if (PendingBlocks.IsEmpty())
					{
						FShaderSource::FViewType Block = SubStrView(Source, 0, Source.Len() - Remainder.Len());
						AddBlock(EBlockType::Unknown, Block);
						ChunkType = ECodeChunkType::CommentLine;
						FinalizeChunk();
					}

					Source = Remainder;

					continue;
				}
				else if (StartsWith(Source, ANSITEXTVIEW("/*")))
				{
					Source = SkipUntilStr(Source, ANSITEXTVIEW("*/"));
					if (Source.Len() >= 2)
					{
						Source = SubStrView(Source, 2);
					}
					continue;
				}
			}
			else if (FirstChar == '#')
			{
				if (StartsWith(Source, ANSITEXTVIEW("#line")))
				{
					FShaderSource::FViewType Remainder = SkipUntilNextLine(Source);
					FShaderSource::FViewType Block = SubStrView(Source, 0, Source.Len() - Remainder.Len());
					Result.LineDirectives.Add(Block);
					Source = Remainder;
					continue;
				}
				else if (StartsWith(Source, ANSITEXTVIEW("#pragma")))
				{
					FShaderSource::FViewType Remainder = SkipUntilNextLine(Source);
					FShaderSource::FViewType Block = SubStrView(Source, 0, Source.Len() - Remainder.Len());
					AddBlock(EBlockType::Directive, Block);
					ChunkType = ECodeChunkType::Pragma;
					FinalizeChunk();
					Source = Remainder;
					continue;
				}
				else if (StartsWith(Source, ANSITEXTVIEW("#define")))
				{
					// TODO: handle `\` new lines in defines
					FShaderSource::FViewType Remainder = SkipUntilNextLine(Source);
					FShaderSource::FViewType Block = SubStrView(Source, 0, Source.Len() - Remainder.Len());
					AddBlock(EBlockType::Directive, Block);
					ChunkType = ECodeChunkType::Define;
					FinalizeChunk();
					Source = Remainder;
					continue;
				}
				else if (StartsWith(Source, ANSITEXTVIEW("#if 0")))
				{
					Source = SkipUntilStr(Source, ANSITEXTVIEW("#endif"));
					if (Source.Len() >= 6)
					{
						Source = SubStrView(Source, 6);
					}
					continue;
				}
			}
			else if (FirstChar == '_' && StartsWith(Source, ANSITEXTVIEW("_Pragma")))
			{
				FShaderSource::FViewType PragmaStart = Source;
				FShaderSource::FViewType Remainder = Source;

				// handle case of multiple _Pragma operators on the same line (to avoid inserting newlines)
				while (PragmaStart.Len() > 0)
				{
					Remainder = SkipUntilStr(Remainder, ANSITEXTVIEW(")"));
					if (Remainder.Len() >= 1)
					{
						Remainder = SubStrView(Remainder, 1);
					}

					FShaderSource::FViewType NextLine = SkipUntilNextLine(Remainder);
					FShaderSource::FViewType LineRemainder = SubStrView(Remainder, 0, Remainder.Len() - NextLine.Len());
					PragmaStart = SkipUntilStr(LineRemainder, ANSITEXTVIEW("_Pragma"));
					if (PragmaStart.Len() > 0)
					{
						Remainder = SkipUntilStr(Remainder, ANSITEXTVIEW("_Pragma"));
					}
				}

				FShaderSource::FViewType Block = SubStrView(Source, 0, Source.Len() - Remainder.Len());
				AddBlock(EBlockType::Directive, Block);
				ChunkType = ECodeChunkType::Pragma;
				FinalizeChunk();
				Source = Remainder;
				continue;
			}
			else if (PendingBlocks.IsEmpty() && (FirstChar == '{' || FirstChar == '}'))
			{
				if (StartsWith(Source, ANSITEXTVIEW("{")))
				{
					if (ChunkType == ECodeChunkType::Namespace)
					{
						if (PendingNamespace.IsEmpty())
						{
							AddDiagnostic(Output.Errors, TEXTVIEW("HLSL does not support anonymous namespaces"));
							break;
						}
						else
						{
							NamespaceTracker.Push(PendingNamespace);
							ChunkType = ECodeChunkType::Unknown;
							PendingNamespace = {};
							Source = Source.Mid(1);
						}
						continue;
					}
					else
					{
						AddDiagnostic(Output.Errors, TEXTVIEW("Expected token '{'"));
					}
					continue;
				}
				else if (StartsWith(Source, ANSITEXTVIEW("}")))
				{
					if (NamespaceTracker.Pop())
					{
						Source = Source.Mid(1);
						continue;
					}
					else
					{
						AddDiagnostic(Output.Errors, TEXTVIEW("Expected token '}'"));
						break;
					}
				}
			}
		}

		if (ChunkType == ECodeChunkType::Operator
			&& !PendingBlocks.IsEmpty()
			&& OperatorKeywordBlockIndex == (PendingBlocks.Num()-1))
		{
			// Operator keyword found in the last processed block. Expect to find operator name character sequence next.

			FShaderSource::FViewType OperatorName = ExtractOperator(Source);

			if (OperatorName.IsEmpty())
			{
				AddDiagnostic(Output.Errors, TEXTVIEW("Unexpected operator overload type"));
				break;
			}

			AddBlock(EBlockType::OperatorName, OperatorName);
			Source = SubStrView(Source, OperatorName.Len());

			continue;
		}

		FShaderSource::FViewType Remainder = SkipUntilNonIdentifierCharacter(Source);
		FShaderSource::FViewType Identifier = SubStrView(Source, 0, Source.Len() - Remainder.Len());

		if (Identifier.Len())
		{
			if (ChunkType == ECodeChunkType::Unknown)
			{
				if (Equals(Identifier, ANSITEXTVIEW("struct")))
				{
					ChunkType = ECodeChunkType::Struct;
					StructBlockIndex = PendingBlocks.Num();
				}
				else if (Equals(Identifier, ANSITEXTVIEW("cbuffer")) || Equals(Identifier, ANSITEXTVIEW("ConstantBuffer")))
				{
					ChunkType = ECodeChunkType::CBuffer;
					CbufferBlockIndex = PendingBlocks.Num();
				}
				else if (Equals(Identifier, ANSITEXTVIEW("enum")))
				{
					ChunkType = ECodeChunkType::Enum;
					EnumBlockIndex = PendingBlocks.Num();
				}
				else if (Equals(Identifier, ANSITEXTVIEW("namespace")))
				{
					ChunkType = ECodeChunkType::Namespace;
					Source = Remainder;
					continue;
				}
				else if (Equals(Identifier, ANSITEXTVIEW("using")))
				{
					ChunkType = ECodeChunkType::Using;
					Source = Remainder;
					AddBlock(EBlockType::Keyword, Identifier);
					continue;
				}
				else if (Equals(Identifier, ANSITEXTVIEW("typedef")))
				{
					ChunkType = ECodeChunkType::Typedef;
					Source = Remainder;
					AddBlock(EBlockType::Keyword, Identifier);
					continue;
				}
				else if (Equals(Identifier, ANSITEXTVIEW("template")))
				{
					Source = Remainder;
					AddBlock(EBlockType::Keyword, Identifier);
					continue;
				}
				else if (Equals(Identifier, ANSITEXTVIEW("operator")))
				{
					ChunkType = ECodeChunkType::Operator;
					Source = Remainder;
					OperatorKeywordBlockIndex = PendingBlocks.Num();
					AddBlock(EBlockType::Keyword, Identifier);
					continue;
				}
			}
			else if (ChunkType == ECodeChunkType::Namespace)
			{
				PendingNamespace = Identifier;
				Source = Remainder;
				continue;
			}

			EBlockType BlockType = EBlockType::Unknown;

			if (bFoundColon)
			{
				if (ChunkType == ECodeChunkType::Struct)
				{
					BlockType = EBlockType::Base;
				}
				else
				{
					BlockType = EBlockType::Binding;
				}

				bFoundColon = false;
			}

			AddBlock(BlockType, Identifier);

			Source = Remainder;
			bFoundIdentifier = true;

			continue;
		}

		FShaderSource::FViewType Block;

		FShaderSource::CharType C = Source[0];

		EBlockType BlockType = EBlockType::Unknown;

		if (StartsWith(Source, ANSITEXTVIEW("==")))
		{
			AddDiagnostic(Output.Errors, TEXTVIEW("Unexpected sequence '=='"));
			break;
		}
		else if (StartsWith(Source, ANSITEXTVIEW("::")))
		{
			Block = SubStrView(Source, 0, 2);
			Source = SubStrView(Source, 2);

			AddBlock(EBlockType::NamespaceDelimiter, Block);

			continue;
		}
		else if (C == '=')
		{
			bFoundAssignment = true;

			Source = SkipSpace(Source.Mid(1));

			char C2 = Source[0];

			if (C2 == '{')
			{
				// extract block on the next loop iteration
				continue;
			}
			else
			{
				int32 Pos = INDEX_NONE;

				if (!Source.FindChar(FShaderSource::CharType(';'), Pos))
				{
					AddDiagnostic(Output.Errors, TEXTVIEW("Expected semicolon after assignment expression"));
					break;
				}

				Block = SubStrView(Source, 0, Pos);
				Block = TrimSpace(Block);

				int32 BlockOffset = int32(Block.GetData() - Source.GetData());
				Source = SubStrView(Source, BlockOffset);

				BlockType = EBlockType::Expression;
				ExpressionBlockIndex = int32(PendingBlocks.Num());
			}
		}
		else if (C == ':')
		{
			Source = SubStrView(Source, 1);
			bFoundColon = true;
			continue;
		}
		else if (C == '(')
		{
			Block = ExtractBlock(Source, '(', ')', Result.LineDirectives);
			BlockType = EBlockType::Args;

			if (ArgsBlockIndex < 0)
			{
				ArgsBlockIndex = PendingBlocks.Num();
			}
		}
		else if (C == '{')
		{
			Block = ExtractBlock(Source, '{', '}', Result.LineDirectives);

			if (BodyBlockIndex == INDEX_NONE && !bFoundAssignment)
			{
				BlockType = EBlockType::Body;
				BodyBlockIndex = PendingBlocks.Num();
				bFoundBody = true;
			}
			else if (bFoundAssignment)
			{
				BlockType = EBlockType::Expression;
				ExpressionBlockIndex = PendingBlocks.Num();
			}
		}
		else if (C == '[')
		{
			Block = ExtractBlock(Source, '[', ']', Result.LineDirectives);

			if (bFoundIdentifier)
			{
				BlockType = EBlockType::Subscript;
			}
			else if (Block.Contains(ANSITEXTVIEW("NodeID")))
			{
				BlockType = EBlockType::NodeId;
			}
			else
			{
				BlockType = EBlockType::Attribute;
			}
		}
		else if (C == '<')
		{
			Block = ExtractBlock(Source, '<', '>', Result.LineDirectives);
			BlockType = EBlockType::TemplateArgs;

			if (ChunkType == ECodeChunkType::CBuffer)
			{
				// `ConstantBuffer<Foo>` is treated as a variable/resource declaration rather than a cbuffer block
				ChunkType = ECodeChunkType::Variable;
			}
		}
		else if (C == ';')
		{
			FinalizeChunk();
			Source = SubStrView(Source, 1);
			continue;
		}
		else if ((C == '*' || C == '&') && !PendingBlocks.IsEmpty()) // Part of a pointer or reference declaration
		{
			Block = SubStrView(Source, 0, 1);
			Source = SubStrView(Source, 1);

			AddBlock(EBlockType::PtrOrRef, Block);

			continue;
		}
		else
		{
			AddDiagnostic(Output.Errors, FString::Printf(TEXT("Unexpected character '%c'"), C));
			break;
		}

		if (Block.IsEmpty())
		{
			AddDiagnostic(Output.Errors, TEXTVIEW("Failed to extract code block"));
			break;
		}
		else
		{
			AddBlock(BlockType, Block);

			Source = SubStrView(Source, Block.Len());

			if (BlockType == EBlockType::Body && ArgsBlockIndex != INDEX_NONE)
			{
				FinalizeChunk();
			}
			else if (BlockType == EBlockType::Body && CbufferBlockIndex != INDEX_NONE)
			{
				FinalizeChunk();
			}
			else if (BlockType == EBlockType::Expression)
			{
				FinalizeChunk();
			}
		}
	}

	Swap(Result.Chunks, Chunks);
	Swap(Result.Namespaces, NamespaceTracker.UniqueNamespaceArray);

	return Result;
}

template<typename CallbackT>
void FindChunksByIdentifier(TConstArrayView<FCodeChunk> Chunks, FShaderSource::FViewType Identifier, CallbackT Callback)
{
	for (const FCodeChunk& Chunk : Chunks)
	{
		for (const FCodeBlock& Block : Chunk.Blocks)
		{
			if (Block == Identifier)
			{
				Callback(Chunk);
			}
		}
	}
}

static TArray<FShaderSource::FViewType> SplitByChar(FShaderSource::FViewType Source, FShaderSource::CharType Delimiter)
{
	TArray<FShaderSource::FViewType> Result;

	int32 Start = 0;
	const int32 SourceLen = Source.Len();

	for (int32 I = 0; I < SourceLen; ++I)
	{
		FShaderSource::CharType C = Source[I];
		if (C == Delimiter)
		{
			size_t Len = I - Start;
			Result.Push(SubStrView(Source, Start, Len));
			Start = I + 1;
		}
	}

	if (Start != Source.Len())
	{
		int32 Len = Source.Len() - Start;
		Result.Push(SubStrView(Source, Start, Len));
	}

	return Result;
}

static void ExtractIdentifiers(FShaderSource::FViewType InSource, TArray<FShaderSource::FViewType, FMemStackAllocator>& Result)
{
	FShaderSource::FViewType Source = InSource;

	while (!Source.IsEmpty())
	{
		FShaderSource::CharType FirstChar = Source.GetData()[0];

		if (FirstChar == '#')
		{
			if (StartsWith(Source, ANSITEXTVIEW("#line"))
				|| StartsWith(Source, ANSITEXTVIEW("#pragma")))
			{
				Source = SkipUntilNextLine(Source);
				Source = SkipSpace(Source);
				continue;
			}
			else if (StartsWith(Source, ANSITEXTVIEW("#if 0")))
			{
				Source = SkipUntilStr(Source, ANSITEXTVIEW("#endif"));
				if (Source.Len() >= 6)
				{
					Source = SubStrView(Source, 6);
				}
				Source = SkipSpace(Source);
				continue;
			}
		}
		else if (FirstChar == '/')
		{
			if (StartsWith(Source, ANSITEXTVIEW("//")))
			{
				Source = SkipUntilNextLine(Source);
				Source = SkipSpace(Source);
				continue;
			}
			else if (StartsWith(Source, ANSITEXTVIEW("/*")))
			{
				Source = SkipUntilStr(Source, ANSITEXTVIEW("*/"));
				if (Source.Len() >= 2)
				{
					Source = SubStrView(Source, 2);
				}
				Source = SkipSpace(Source);
				continue;
			}
		}

		FShaderSource::FViewType Remainder = SkipUntilNonIdentifierCharacter(Source);

		FShaderSource::FViewType Identifier = SubStrView(Source, 0, Source.Len() - Remainder.Len());

		if (Identifier.IsEmpty())
		{
			if (!Remainder.IsEmpty())
			{
				Remainder = SubStrView(Remainder, 1);
			}
		}
		else
		{
			if (!IsNumber(Identifier[0]))  // Identifiers can't start with numbers
			{
				Result.Push(Identifier);
			}
		}

		Source = SkipSpace(Remainder);
	}
}

static void ExtractIdentifiers(const FCodeChunk& Chunk, TArray<FShaderSource::FViewType, FMemStackAllocator>& Result)
{
	for (const FCodeBlock& Block : Chunk.Blocks)
	{
		ExtractIdentifiers(Block, Result);
	}
}

static void OutputChunk(const FCodeChunk& Chunk, FShaderSource::FStringType& OutputStream)
{
	if (Chunk.Blocks.IsEmpty())
	{
		return;
	}

	if (Chunk.bVerbatim)
	{
		// Fast path to output entire code block verbatim, preserving any new lines and whitespace		
		OutputStream.Append(Chunk.GetCode());
	}
	else
	{
		int32 Index = 0;
		for (const FCodeBlock& Block : Chunk.Blocks)
		{
			if (Index != 0)
			{
				OutputStream.AppendChar(' ');
			}

			if (Block.Type == EBlockType::Expression)
			{
				OutputStream.Append(ANSITEXTVIEW("= "));
			}
			else if (Block.Type == EBlockType::Body)
			{
				OutputStream.AppendChar('\n');
			}

			if (Block.Type == EBlockType::Binding || Block.Type == EBlockType::Base)
			{
				OutputStream.Append(ANSITEXTVIEW(": "));
			}

			OutputStream.Append(Block.GetCode());

			++Index;
		}
	}

	if (Chunk.Type != ECodeChunkType::Function
		&& Chunk.Type != ECodeChunkType::Operator
		&& Chunk.Type != ECodeChunkType::CBuffer
		&& Chunk.Type != ECodeChunkType::Pragma
		&& Chunk.Type != ECodeChunkType::Define
		&& Chunk.Type != ECodeChunkType::CommentLine)
	{
		OutputStream.AppendChar(';');
	}

	OutputStream.AppendChar('\n');
}

struct FCasedStringViewKeyFuncs : public DefaultKeyFuncs<FShaderSource::FViewType>
{
	static FORCEINLINE FShaderSource::FViewType GetSetKey(FShaderSource::FViewType K) { return K; }
	template <typename T>
	static FORCEINLINE FShaderSource::FViewType GetSetKey(const TPair<FShaderSource::FViewType, T>& P) { return P.Key; }
	static FORCEINLINE bool Matches(FShaderSource::FViewType A, FShaderSource::FViewType B) { return Equals(A, B); }
	static FORCEINLINE uint32 GetKeyHash(FShaderSource::FViewType Key)
	{
		return FXxHash64::HashBuffer(Key.GetData(), Key.Len() * sizeof(*Key.GetData())).Hash;
	}
};

static void BuildLineBreakMap(FShaderSource::FViewType Source, TArray<int32, FMemStackAllocator>& OutLineBreakMap)
{
	OutLineBreakMap.Reset();

	OutLineBreakMap.Add(0); // Lines numbers are 1-based, so add a dummy element to make UpperBound later return the line number directly

	const int32 SourceLen = Source.Len();
	const FShaderSource::CharType* Chars = Source.GetData(); // avoid bounds check overhead in [] operator

	int32 Cursor = 0;

#if UE_SHADER_MINIFIER_SSE
	const __m128i Needle = _mm_set1_epi8('\n');
	while (Cursor < SourceLen)
	{
		__m128i Chunk = _mm_loadu_si128(reinterpret_cast<const __m128i*>(Chars + Cursor));
		__m128i MaskVec = _mm_cmpeq_epi8(Chunk, Needle);
		uint32 Mask = _mm_movemask_epi8(MaskVec);
		while (Mask != 0)
		{
			const uint32 BitIndex = FMath::CountTrailingZeros(Mask);
			const uint32 ChunkCharIndex = BitIndex / sizeof(FShaderSource::CharType);
			OutLineBreakMap.Add(Cursor + ChunkCharIndex);
			Mask &= ~(1u << BitIndex);
		}
		Cursor += 16;
	}
#else
	while (Cursor < SourceLen)
	{
		if (Chars[Cursor] == FShaderSource::CharType('\n'))
		{
			OutLineBreakMap.Add(Cursor);
		}
		++Cursor;
	}
#endif //UE_SHADER_MINIFIER_SSE
}

static int32 FindLineDirective(const TArray<FShaderSource::FViewType>& LineDirectives, const FShaderSource::CharType* Ptr)
{
	int32 FoundIndex = Algo::UpperBoundBy(LineDirectives, Ptr, [](FShaderSource::FViewType Item)
	{
		return Item.GetData();
	});

	if (FoundIndex < 1 || FoundIndex > LineDirectives.Num())
	{
		return INDEX_NONE;
	}

	// UpperBound returns element that's greater than predicate, but we need the closest preceeding line directive.
	return FoundIndex - 1;
}

static int32 FindLineNumber(FShaderSource::FViewType Source, const TArray<int32, FMemStackAllocator>& LineBreakMap, const FShaderSource::CharType* Ptr)
{
	if (Ptr < Source.GetData() || Ptr >= Source.GetData() + Source.Len())
	{
		return INDEX_NONE;
	}

	const int32 Index = int32(Ptr - Source.GetData());

	int32 FoundLineNumber = Algo::UpperBound(LineBreakMap, Index);

	return FoundLineNumber;
}

static bool ParseLineDirective(FShaderSource::FViewType Input, int32& OutLineNumber, FShaderSource::FViewType& OutFileName)
{
	if (!StartsWith(Input, ANSITEXTVIEW("#line")))
	{
		return false;
	}

	Input = Input.Mid(5); // skip `#line` itself
	Input = SkipSpace(Input);

	if (Input.IsEmpty() || !IsNumber(Input[0]))
	{
		return false;
	}

	OutLineNumber = FShaderSource::FCStringType::Atoi(Input.GetData());

	int32 FileNameBeginIndex = INDEX_NONE;
	if (Input.FindChar(FShaderSource::CharType('"'), FileNameBeginIndex))
	{
		int32 FileNameEndIndex = INDEX_NONE;
		Input.MidInline(FileNameBeginIndex + 1);
		if (Input.FindChar(FShaderSource::CharType('"'), FileNameEndIndex))
		{
			OutFileName = Input.Mid(0, FileNameEndIndex);
		}
		else
		{
			return false;
		}
	}

	return true;
}

static void OpenNamespace(FShaderSource::FStringType& OutputStream, const FNamespace& Namespace)
{
	for (const FShaderSource::FViewType& Name : Namespace.Stack)
	{
		OutputStream.Append(ANSITEXTVIEW("namespace "));
		OutputStream.Append(Name);
		OutputStream.Append(ANSITEXTVIEW(" { "));
	}
}

static void CloseNamespace(FShaderSource::FStringType& OutputStream, const FNamespace& Namespace)
{
	for (const FShaderSource::FViewType& Name : Namespace.Stack)
	{
		OutputStream.AppendChar('}');
	}

	OutputStream.Append(ANSITEXTVIEW(" // namespace "));
	OutputStream.Append(Namespace.FullName);
}

static FShaderSource::FStringType MinifyShader(const FParsedShader& Parsed, TConstArrayView<FShaderSource::FViewType> RequiredSymbols, EMinifyShaderFlags Flags, FDiagnostics& Diagnostics)
{
	FMemMark Mark(FMemStack::Get());

	FShaderSource::FStringType OutputStream;

	OutputStream.Reserve(Parsed.Source.Len() / 3); // Heuristic pre-allocation based on average measured reduced code size

	TSet<FShaderSource::FViewType, FCasedStringViewKeyFuncs, FMemStackSetAllocator> RelevantIdentifiers;

	TSet<const FCodeChunk*, DefaultKeyFuncs<const FCodeChunk*>, FMemStackSetAllocator> RelevantChunks;
	TSet<FShaderSource::FViewType, FCasedStringViewKeyFuncs, FMemStackSetAllocator> ProcessedIdentifiers;

	TArray<const FCodeChunk*, FMemStackAllocator> PendingChunks;

	for (FShaderSource::FViewType Entry : RequiredSymbols)
	{
		RelevantIdentifiers.Add(Entry);
		ProcessedIdentifiers.Add(Entry);
		FindChunksByIdentifier(Parsed.Chunks, Entry, [&PendingChunks](const FCodeChunk& Chunk) { PendingChunks.Push(&Chunk); });
	}

	for (const FCodeChunk* Chunk : PendingChunks)
	{
		RelevantChunks.Add(Chunk);
	}

	{
		// Some known builtin words to ignore
		ProcessedIdentifiers.Add("asfloat");
		ProcessedIdentifiers.Add("asint");
		ProcessedIdentifiers.Add("asuint");
		ProcessedIdentifiers.Add("bool");
		ProcessedIdentifiers.Add("bool2");
		ProcessedIdentifiers.Add("bool3");
		ProcessedIdentifiers.Add("bool4");
		ProcessedIdentifiers.Add("break");
		ProcessedIdentifiers.Add("cbuffer");
		ProcessedIdentifiers.Add("const");
		ProcessedIdentifiers.Add("else");
		ProcessedIdentifiers.Add("extern");
		ProcessedIdentifiers.Add("false");
		ProcessedIdentifiers.Add("float");
		ProcessedIdentifiers.Add("float2");
		ProcessedIdentifiers.Add("float3");
		ProcessedIdentifiers.Add("float3x3");
		ProcessedIdentifiers.Add("float3x4");
		ProcessedIdentifiers.Add("float4");
		ProcessedIdentifiers.Add("float4x4");
		ProcessedIdentifiers.Add("for");
		ProcessedIdentifiers.Add("groupshared");
		ProcessedIdentifiers.Add("if");
		ProcessedIdentifiers.Add("in");
		ProcessedIdentifiers.Add("inout");
		ProcessedIdentifiers.Add("int");
		ProcessedIdentifiers.Add("int2");
		ProcessedIdentifiers.Add("int3");
		ProcessedIdentifiers.Add("int4");
		ProcessedIdentifiers.Add("interface");
		ProcessedIdentifiers.Add("out");
		ProcessedIdentifiers.Add("packoffset");
		ProcessedIdentifiers.Add("precise");
		ProcessedIdentifiers.Add("register");
		ProcessedIdentifiers.Add("return");
		ProcessedIdentifiers.Add("static");
		ProcessedIdentifiers.Add("struct");
		ProcessedIdentifiers.Add("switch");
		ProcessedIdentifiers.Add("tbuffer");
		ProcessedIdentifiers.Add("true");
		ProcessedIdentifiers.Add("uint");
		ProcessedIdentifiers.Add("uint2");
		ProcessedIdentifiers.Add("uint3");
		ProcessedIdentifiers.Add("uint4");
		ProcessedIdentifiers.Add("void");
		ProcessedIdentifiers.Add("while");
		ProcessedIdentifiers.Add("typedef");
		ProcessedIdentifiers.Add("template");
		ProcessedIdentifiers.Add("operator");
		ProcessedIdentifiers.Add("enum");

		// HLSL resource types
		ProcessedIdentifiers.Add("TextureCubeArray");
		ProcessedIdentifiers.Add("TextureCube");
		ProcessedIdentifiers.Add("TextureBuffer");
		ProcessedIdentifiers.Add("Texture3D");
		ProcessedIdentifiers.Add("Texture2DMSArray");
		ProcessedIdentifiers.Add("Texture2DMS");
		ProcessedIdentifiers.Add("Texture2DArray");
		ProcessedIdentifiers.Add("Texture2D");
		ProcessedIdentifiers.Add("Texture1DArray");
		ProcessedIdentifiers.Add("Texture1D");
		ProcessedIdentifiers.Add("StructuredBuffer");
		ProcessedIdentifiers.Add("SamplerState");
		ProcessedIdentifiers.Add("SamplerComparisonState");
		ProcessedIdentifiers.Add("RWTextureCubeArray");
		ProcessedIdentifiers.Add("RWTextureCube");
		ProcessedIdentifiers.Add("RWTexture3D");
		ProcessedIdentifiers.Add("RWTexture2DMSArray");
		ProcessedIdentifiers.Add("RWTexture2DMS");
		ProcessedIdentifiers.Add("RWTexture2DArray");
		ProcessedIdentifiers.Add("RWTexture2D");
		ProcessedIdentifiers.Add("RWTexture1DArray");
		ProcessedIdentifiers.Add("RWTexture1D");
		ProcessedIdentifiers.Add("RWStructuredBuffer");
		ProcessedIdentifiers.Add("RWByteAddressBuffer");
		ProcessedIdentifiers.Add("RWBuffer");
		ProcessedIdentifiers.Add("RaytracingAccelerationStructure");
		ProcessedIdentifiers.Add("RasterizerOrderedTexture3D");
		ProcessedIdentifiers.Add("RasterizerOrderedTexture2DArray");
		ProcessedIdentifiers.Add("RasterizerOrderedTexture2D");
		ProcessedIdentifiers.Add("RasterizerOrderedTexture1DArray");
		ProcessedIdentifiers.Add("RasterizerOrderedTexture1D");
		ProcessedIdentifiers.Add("RasterizerOrderedStructuredBuffer");
		ProcessedIdentifiers.Add("RasterizerOrderedByteAddressBuffer");
		ProcessedIdentifiers.Add("RasterizerOrderedBuffer");
		ProcessedIdentifiers.Add("FeedbackTexture2DArray");
		ProcessedIdentifiers.Add("FeedbackTexture2D");
		ProcessedIdentifiers.Add("ConsumeStructuredBuffer");
		ProcessedIdentifiers.Add("ConstantBuffer");
		ProcessedIdentifiers.Add("ByteAddressBuffer");
		ProcessedIdentifiers.Add("Buffer");
		ProcessedIdentifiers.Add("AppendStructuredBuffer");

		// Alternative spelling of some resource types
		ProcessedIdentifiers.Add("AppendRegularBuffer");
		ProcessedIdentifiers.Add("ByteBuffer");
		ProcessedIdentifiers.Add("ConsumeRegularBuffer");
		ProcessedIdentifiers.Add("DataBuffer");
		ProcessedIdentifiers.Add("MS_Texture2D");
		ProcessedIdentifiers.Add("MS_Texture2D_Array");
		ProcessedIdentifiers.Add("RegularBuffer");
		ProcessedIdentifiers.Add("RW_ByteBuffer");
		ProcessedIdentifiers.Add("RW_DataBuffer");
		ProcessedIdentifiers.Add("RW_RegularBuffer");
		ProcessedIdentifiers.Add("RW_Texture1D");
		ProcessedIdentifiers.Add("RW_Texture1D_Array");
		ProcessedIdentifiers.Add("RW_Texture2D");
		ProcessedIdentifiers.Add("RW_Texture2D_Array");
		ProcessedIdentifiers.Add("RW_Texture3D");
		ProcessedIdentifiers.Add("RW_TextureCube");
		ProcessedIdentifiers.Add("Texture1D_Array");
		ProcessedIdentifiers.Add("Texture2D_Array");
		ProcessedIdentifiers.Add("TextureBuffer");
		ProcessedIdentifiers.Add("TextureCube_Array");

		// Some shaders define template versions of some built-in functions, so we can't trivially ignore them
		//ProcessedIdentifiers.Add("abs");
		//ProcessedIdentifiers.Add("any");
		//ProcessedIdentifiers.Add("clamp");
		//ProcessedIdentifiers.Add("clip");
		//ProcessedIdentifiers.Add("cos");
		//ProcessedIdentifiers.Add("cross");
		//ProcessedIdentifiers.Add("dot");
		//ProcessedIdentifiers.Add("frac");
		//ProcessedIdentifiers.Add("lerp");
		//ProcessedIdentifiers.Add("max");
		//ProcessedIdentifiers.Add("min");
		//ProcessedIdentifiers.Add("mul");
		//ProcessedIdentifiers.Add("normalize");
		//ProcessedIdentifiers.Add("pow");
		//ProcessedIdentifiers.Add("saturate");
		//ProcessedIdentifiers.Add("sign");
		//ProcessedIdentifiers.Add("sin");
		//ProcessedIdentifiers.Add("sqrt");
	}

	if (PendingChunks.IsEmpty())
	{
		// Entry point chunk is not found in the shader
		return {};
	}

	TArray<FShaderSource::FViewType, FMemStackAllocator> TempIdentifiers;

	TMap<FShaderSource::FViewType, TArray<const FCodeChunk*, FMemStackAllocator>, FMemStackSetAllocator, FCasedStringViewKeyFuncs> ChunksByIdentifier;
	for (const FCodeChunk& Chunk : Parsed.Chunks)
	{
		for (const FCodeBlock& Block : Chunk.Blocks)
		{
			if (Block.Type == EBlockType::Keyword)
			{
				continue;
			}

			if (Chunk.Type == ECodeChunkType::Function && Block.Type != EBlockType::Name && Block.Type != EBlockType::NodeId)
			{
				continue;
			}

			if (Chunk.Type == ECodeChunkType::Struct && Block.Type != EBlockType::Type)
			{
				continue;
			}

			if (Chunk.Type == ECodeChunkType::Typedef && Block.Type != EBlockType::Name)
			{
				continue;
			}

			if (Chunk.Type == ECodeChunkType::Operator
				&& Block.Type != EBlockType::Type
				&& Block.Type != EBlockType::Args)
			{
				continue;
			}

			bool bExtractIdentifiers = (Chunk.Type == ECodeChunkType::CBuffer && Block.Type == EBlockType::Body);
			bExtractIdentifiers |= (Chunk.Type == ECodeChunkType::Enum && Block.Type == EBlockType::Body);
			bExtractIdentifiers |= (Chunk.Type == ECodeChunkType::Function && Block.Type == EBlockType::NodeId);

			if (bExtractIdentifiers)
			{
				TempIdentifiers.Reset();
				ExtractIdentifiers(Block, TempIdentifiers);
				for (FShaderSource::FViewType Identifier : TempIdentifiers)
				{
					ChunksByIdentifier.FindOrAdd(Identifier).Push(&Chunk);
				}

				continue;
			}

			ChunksByIdentifier.FindOrAdd(Block).Push(&Chunk);
		}
	}

	TMap<const FCodeChunk*, const FCodeChunk*, FMemStackSetAllocator> ChunkRequestedBy;

	while (!PendingChunks.IsEmpty())
	{
		TempIdentifiers.Reset();

		const FCodeChunk* CurrentChunk = PendingChunks.Last();
		PendingChunks.Pop();

		ExtractIdentifiers(*CurrentChunk, TempIdentifiers);

		for (FShaderSource::FViewType Identifier : TempIdentifiers)
		{
			bool bIdentifierWasAlreadyInSet = false;
			ProcessedIdentifiers.Add(Identifier, &bIdentifierWasAlreadyInSet);
			if (!bIdentifierWasAlreadyInSet)
			{
				auto FoundChunks = ChunksByIdentifier.Find(Identifier);
				if (FoundChunks == nullptr)
				{
					continue;
				}
				for (const FCodeChunk* Chunk : *FoundChunks)
				{
					if (Chunk == CurrentChunk)
					{
						continue;
					}

					bool bChunkWasAlreadyInSet = false;
					RelevantChunks.Add(Chunk, &bChunkWasAlreadyInSet);
					if (!bChunkWasAlreadyInSet)
					{
						PendingChunks.Push(Chunk);

						if (Chunk->Type == ECodeChunkType::Function 
							|| Chunk->Type == ECodeChunkType::Struct
							|| Chunk->Type == ECodeChunkType::CBuffer
							|| Chunk->Type == ECodeChunkType::Variable)
						{
							ChunkRequestedBy.FindOrAdd(Chunk) = CurrentChunk;
						}
					}
				}
			}
		}
	}

	uint32 NumFunctions		= 0;
	uint32 NumStructs		= 0;
	uint32 NumVariables		= 0;
	uint32 NumCBuffers		= 0;
	uint32 NumOtherChunks	= 0;

	for (const FCodeChunk& Chunk : Parsed.Chunks)
	{
		if (RelevantChunks.Find(&Chunk) == nullptr)
		{
			continue;
		}

		if (Chunk.Type == ECodeChunkType::Function)
		{
			NumFunctions += 1;
		}
		else if (Chunk.Type == ECodeChunkType::Struct)
		{
			NumStructs += 1;
		}
		else if (Chunk.Type == ECodeChunkType::Variable)
		{
			NumVariables += 1;
		}
		else if (Chunk.Type == ECodeChunkType::CBuffer)
		{
			NumCBuffers += 1;
		}
		else
		{
			NumOtherChunks += 1;
		}
	}

	if (EnumHasAnyFlags(Flags, EMinifyShaderFlags::OutputStats))
	{
		OutputStream.Append(ANSITEXTVIEW("// Total code chunks: "));
		OutputStream.AppendInt(RelevantChunks.Num());
		OutputStream.AppendChar('\n');

		OutputStream.Append(ANSITEXTVIEW("// - Functions: "));
		OutputStream.AppendInt(NumFunctions);
		OutputStream.AppendChar('\n');

		OutputStream.Append(ANSITEXTVIEW("// - Structs: "));
		OutputStream.AppendInt(NumStructs);
		OutputStream.AppendChar('\n');

		OutputStream.Append(ANSITEXTVIEW("// - CBuffers: "));
		OutputStream.AppendInt(NumCBuffers);
		OutputStream.AppendChar('\n');

		OutputStream.Append(ANSITEXTVIEW("// - Variables: "));
		OutputStream.AppendInt(NumVariables);
		OutputStream.AppendChar('\n');

		OutputStream.Append(ANSITEXTVIEW("// - Other: "));
		OutputStream.AppendInt(NumOtherChunks);
		OutputStream.AppendChar('\n');

		OutputStream.AppendChar('\n');
	}

	TArray<int32, FMemStackAllocator> LineBreakMap;
	const TArray<FShaderSource::FViewType>& LineDirectives = Parsed.LineDirectives;
	if (EnumHasAnyFlags(Flags, EMinifyShaderFlags::OutputLines))
	{
		BuildLineBreakMap(Parsed.Source, LineBreakMap);
	}

	const FNamespace* CurrentNamespace = nullptr;

	int32 LastLineNumber = -1;
	FShaderSource::FViewType LastLineFileName;

	for (const FCodeChunk& Chunk : Parsed.Chunks)
	{
		auto ShouldSkipChunk = [&RelevantChunks, &Chunk, Flags]()
		{
			// Pragmas and defines that remain after preprocessing must be preserved as they may control important compiler behaviors.
			if (Chunk.Type == ECodeChunkType::Pragma || Chunk.Type == ECodeChunkType::Define)
			{
				return false;
			}

			// The preprocessed shader may have auto-generated comments such as `// #define FOO 123` that may be useful to keep for debugging.
			if (Chunk.Type == ECodeChunkType::CommentLine && EnumHasAnyFlags(Flags, EMinifyShaderFlags::OutputCommentLines))
			{
				return false;
			}

			// Always include `using` statements if they are present in the global scope
			if (Chunk.Type == ECodeChunkType::Using)
			{
				return false;
			}

			if (RelevantChunks.Find(&Chunk))
			{
				return false;
			}

			return true;
		};

		if (ShouldSkipChunk())
		{
			continue;
		}

		const FNamespace* PendingNamespace = Chunk.Namespace != INDEX_NONE ? &Parsed.Namespaces[Chunk.Namespace] : nullptr;

		if (PendingNamespace != CurrentNamespace)
		{
			if (CurrentNamespace)
			{
				CloseNamespace(OutputStream, *CurrentNamespace);
				OutputStream.Append(ANSITEXTVIEW("\n\n"));
			}

			if (PendingNamespace)
			{
				OpenNamespace(OutputStream, *PendingNamespace);
				OutputStream.Append(ANSITEXTVIEW("\n\n"));
			}

			CurrentNamespace = PendingNamespace;
		}

		if (EnumHasAnyFlags(Flags, EMinifyShaderFlags::OutputReasons))
		{
			auto RequestedBy = ChunkRequestedBy.Find(&Chunk);
			if (RequestedBy != nullptr)
			{
				const FCodeChunk* RequestedByChunk = *RequestedBy;
				FShaderSource::FViewType  RequestedByName = RequestedByChunk->FindFirstBlockByType(EBlockType::Name);
				if (!RequestedByName.IsEmpty())
				{
					OutputStream.Append(ANSITEXTVIEW("// REASON: "));
					OutputStream.Append(RequestedByName);
					OutputStream.AppendChar('\n');
				}
			}
		}

		if (EnumHasAnyFlags(Flags, EMinifyShaderFlags::OutputLines))
		{
			const FShaderSource::FViewType ChunkCode = Chunk.Blocks[0];
			int32 LineDirectiveIndex = FindLineDirective(LineDirectives, ChunkCode.GetData());
			int32 ChunkLine = FindLineNumber(Parsed.Source, LineBreakMap, ChunkCode.GetData());

			if (ChunkLine != INDEX_NONE && LineDirectiveIndex == INDEX_NONE)
			{
				// There was no valid line directive for this chunk, but we do know the line in the input source, so just emit that.
				if (ChunkLine > LastLineNumber + 1 || !LastLineFileName.IsEmpty())
				{
					OutputStream.Append(ANSITEXTVIEW("#line "));
					OutputStream.AppendInt(ChunkLine);
					OutputStream.AppendChar('\n');
				}

				LastLineNumber = ChunkLine;
				LastLineFileName = FShaderSource::FViewType();
			}
			else if (ChunkLine != INDEX_NONE && LineDirectiveIndex != INDEX_NONE)
			{
				// We have a valid line directive and line number in the input source.
				// Some of the input source code may have been removed, so we need to adjust 
				// the line number before emitting the line directive.

				FShaderSource::FViewType LineDirective = LineDirectives[LineDirectiveIndex];
				int32 LineDirectiveLine = FindLineNumber(Parsed.Source, LineBreakMap, LineDirective.GetData());
				int32 ParsedLineNumber = INDEX_NONE;
				FShaderSource::FViewType ParsedFileName;
				if (LineDirectiveLine != INDEX_NONE && ParseLineDirective(LineDirective, ParsedLineNumber, ParsedFileName))
				{
					int32 OffsetFromLineDirective = ChunkLine - (LineDirectiveLine + 1); // Line directive identifies the *next* line, hence +1 when computing the offset
					int32 PatchedLineNumber = ParsedLineNumber + OffsetFromLineDirective;

					if (PatchedLineNumber > LastLineNumber + 1 || LastLineFileName != ParsedFileName)
					{
						// Separate the next block from the previous one when it starts with a line directive
						if (OutputStream.Len())
						{
							OutputStream.AppendChar('\n');
						}

						OutputStream.Append(ANSITEXTVIEW("#line "));
						OutputStream.AppendInt(PatchedLineNumber);

						if (!ParsedFileName.IsEmpty())
						{
							OutputStream.Append(ANSITEXTVIEW(" \""));
							OutputStream.Append(ParsedFileName);
							OutputStream.AppendChar('\"');
						}
						OutputStream.AppendChar('\n');
					}

					LastLineNumber = PatchedLineNumber;
					LastLineFileName = ParsedFileName;
				}
			}
		}

		OutputChunk(Chunk, OutputStream);
	}

	if (CurrentNamespace)
	{
		CloseNamespace(OutputStream, *CurrentNamespace);
		OutputStream.AppendChar('\n');
		CurrentNamespace = nullptr;
	}

	return OutputStream;
}

static FShaderSource::FStringType MinifyShader(const FParsedShader& Parsed, FShaderSource::FViewType EntryPoint, EMinifyShaderFlags Flags, FDiagnostics& Diagnostics)
{
	TArray<FShaderSource::FViewType> RequiredSymbols = SplitByChar(EntryPoint, ';');
	return MinifyShader(Parsed, RequiredSymbols, Flags, Diagnostics);
}

FMinifiedShader Minify(const FShaderSource& PreprocessedShader, TConstArrayView<FShaderSource::FViewType> RequiredSymbols, EMinifyShaderFlags Flags)
{
	FMinifiedShader Result;

	FParsedShader Parsed = ParseShader(PreprocessedShader, Result.Diagnostics);

	if (!Parsed.Chunks.IsEmpty())
	{
		Result.Code = MinifyShader(Parsed, RequiredSymbols, Flags, Result.Diagnostics);
	}

	return Result;
}

FMinifiedShader Minify(const FShaderSource& PreprocessedShader, const FShaderSource::FViewType EntryPoint, EMinifyShaderFlags Flags)
{
	return Minify(PreprocessedShader, MakeArrayView(&EntryPoint, 1), Flags);
}

} // namespace UE::ShaderMinifier

#if WITH_LOW_LEVEL_TESTS

namespace UE::ShaderMinifier
{
// Convenience wrapper for tests where we don't care about diagnostic messages
static FParsedShader ParseShader(const FShaderSource& InSource)
{
	FDiagnostics Diagnostics;
	return ParseShader(InSource, Diagnostics);
}
}

TEST_CASE_NAMED(FShaderMinifierParserTest, "Shaders::ShaderMinifier::Parse", "[EditorContext][EngineFilter]")
{
	using namespace UE::ShaderMinifier;

	{
		FShaderSource S("  \n\r\f \tHello");
		TEST_EQUAL(TEXT("SkipSpace"),
			FString(SkipSpace(S.GetView())),
			FString("Hello"));
	}
	{
		FShaderSource S("Hello World");
		TEST_EQUAL(TEXT("SkipUntilStr (found)"),
			FString(SkipUntilStr(S.GetView(), "World")),
			FString("World"));
	}
	{
		FShaderSource S("Hello World");
		TEST_EQUAL(TEXT("SkipUntilStr (not found)"),
			FString(SkipUntilStr(S.GetView(), "Blah")),
			FString());
	}

	{
		FShaderSource S("static const struct { int Blah; } Foo = { 123; };");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("Anonymous struct variable with initializer, total chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("Anonymous struct variable with initializer, main chunk type"), P.Chunks[0].Type, ECodeChunkType::Variable);
		}
	}

	{
		FShaderSource S("float4 PSMain() : SV_Target { return float4(1,0,0,1); };");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("Pixel shader entry point, total chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("Pixel shader entry point, main chunk type"), P.Chunks[0].Type, ECodeChunkType::Function);
		}
	}

	{
		FMemMark Mark(FMemStack::Get());
		TArray<FShaderSource::FViewType, FMemStackAllocator> R;
		FShaderSource S("Hello[World]; Foo[0];\n");
		ExtractIdentifiers(S.GetView(), R);
		TEST_IF_EQUAL(TEXT("ExtractIdentifiers1: Num"), R.Num(), 3)
		{
			TEST_EQUAL(TEXT("ExtractIdentifiers1: R[0]"), FString(R[0]), "Hello");
			TEST_EQUAL(TEXT("ExtractIdentifiers1: R[1]"), FString(R[1]), "World");
			TEST_EQUAL(TEXT("ExtractIdentifiers1: R[2]"), FString(R[2]), "Foo");
		}
	}

	{
		FMemMark Mark(FMemStack::Get());
		TArray<FShaderSource::FViewType, FMemStackAllocator> R;
		FShaderSource S("#line 0\nStructuredBuffer<uint4> Blah : register(t0, space123);#line 1\n#pragma foo\n");
		ExtractIdentifiers(S.GetView(), R);
		TEST_IF_EQUAL(TEXT("ExtractIdentifiers2: Num"), R.Num(), 6)
		{
			TEST_EQUAL(TEXT("ExtractIdentifiers2: R[0]"), FString(R[0]), "StructuredBuffer");
			TEST_EQUAL(TEXT("ExtractIdentifiers2: R[1]"), FString(R[1]), "uint4");
			TEST_EQUAL(TEXT("ExtractIdentifiers2: R[2]"), FString(R[2]), "Blah");
			TEST_EQUAL(TEXT("ExtractIdentifiers2: R[3]"), FString(R[3]), "register");
			TEST_EQUAL(TEXT("ExtractIdentifiers2: R[4]"), FString(R[4]), "t0");
			TEST_EQUAL(TEXT("ExtractIdentifiers2: R[5]"), FString(R[5]), "space123");
		}
	}

	{
		FShaderSource S("StructuredBuffer<uint4> Blah : register(t0, space123);");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: structured buffer: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: structured buffer: chunk"), P.Chunks[0].Type, ECodeChunkType::Variable);
		}
	}

	{
		FShaderSource S("const float Foo = 123.45f;");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: const float with initializer: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: const float with initializer: chunk type"), P.Chunks[0].Type, ECodeChunkType::Variable);
		}
	}

	{
		FShaderSource S("struct Blah { int A; };");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: struct: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: struct: chunk type"), P.Chunks[0].Type, ECodeChunkType::Struct);
		}
	}

	{
		FShaderSource S("struct Foo { int FooA; }; struct Bar : Foo { int BarA; };");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: inherited struct: num chunks"), P.Chunks.Num(), 2)
		{
			TEST_EQUAL(TEXT("ParseShader: inherited struct: chunk 0 type"), P.Chunks[0].Type, ECodeChunkType::Struct);
			TEST_EQUAL(TEXT("ParseShader: inherited struct: chunk 1 type"), P.Chunks[1].Type, ECodeChunkType::Struct);
		}
	}

	{
		FShaderSource S("[numthreads(8,8,1)] void Main() {};");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: compute shader entry point: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: compute shader entry point: chunk type"), P.Chunks[0].Type, ECodeChunkType::Function);

			TEST_IF_EQUAL(TEXT("ParseShader: compute shader entry point: num blocks"), P.Chunks[0].Blocks.Num(), 5)
			{
				TEST_EQUAL(TEXT("ParseShader: compute shader entry point: attribute block type"), P.Chunks[0].Blocks[0].Type, EBlockType::Attribute);
			}
		}
	}

	{
		FShaderSource S("Texture2D Blah : register(t0);");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: texture with register: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: texture with register: chunk type"), P.Chunks[0].Type, ECodeChunkType::Variable);
		}
	}
	{
		FShaderSource S("Texture2D Blah;");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: texture: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: texture: chunk type"), P.Chunks[0].Type, ECodeChunkType::Variable);
		}
	}

	{
		FShaderSource S("SamplerState Blah : register(s0, space123);");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: sampler state with register: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: sampler state with register: chunk type"), P.Chunks[0].Type, ECodeChunkType::Variable);
		}
	}

#if 0
	{
		// TODO: handle function forward declarations
		FShaderSource S("Foo Fun(int a);");
		auto P = ParseShader(S);
		TEST_EQUAL(TEXT("ParseShader: function forward declaration"), P.Chunks[0].Type, ECodeChunkType::FunctionDecl);
	}
#endif

	{
		FShaderSource S("void Fun(int a) {};");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: function with trailing semicolon: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: function with trailing semicolon: chunk type"), P.Chunks[0].Type, ECodeChunkType::Function);
		}
	}

	{
		FShaderSource S("void Fun(int a) {}");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: function: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: function: chunk type"), P.Chunks[0].Type, ECodeChunkType::Function);
		}
	}

	{
		FShaderSource S("cbuffer Foo {blah} SamplerState S;");
		auto P = ParseShader(S);

		TEST_IF_EQUAL(TEXT("ParseShader: cbuffer and sampler state: num chunks"), P.Chunks.Num(), 2)
		{
			TEST_EQUAL(TEXT("ParseShader: cbuffer and sampler state: chunk type [0]"), P.Chunks[0].Type, ECodeChunkType::CBuffer);
			TEST_EQUAL(TEXT("ParseShader: cbuffer and sampler state: chunk type [1]"), P.Chunks[1].Type, ECodeChunkType::Variable);
		}
	}

	{
		FShaderSource S("struct Foo { int a; };");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: struct: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: struct: chunk type"), P.Chunks[0].Type, ECodeChunkType::Struct);
		}
	}

	{
		FShaderSource S("struct { int a; } Foo = { 123; };");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: anonymous struct with variable and initializer: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: anonymous struct with variable and initializer: chunk type [0]"), P.Chunks[0].Type, ECodeChunkType::Variable);
		}
	}

#if 0
	{
		// TODO: handle struct forward declarations
		FShaderSource S("struct Foo;");
		auto P = ParseShader(S);
	}
#endif

	{
		FShaderSource S(
				"cbuffer MyBuffer : register(b3)"
				"{ float4 Element1 : packoffset(c0); float1 Element2 : packoffset(c1); float1 Element3 : packoffset(c1.y); }"
			);
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: cbuffer with packoffset: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: cbuffer with packoffset: chunk type"), P.Chunks[0].Type, ECodeChunkType::CBuffer);
		}
	}

	{
		FShaderSource S("static const struct { float4 Param; } Foo;");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: static const anonymous struct with variable: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: static const anonymous struct with variable: chunk type"), P.Chunks[0].Type, ECodeChunkType::Variable);
		}
	}

	{
		FShaderSource S("static const struct { float4 Param; } Foo = { FooCB_Param; };");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: static const anonymous struct with variable and initializer: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: static const anonymous struct with variable and initializer: chunk type"), P.Chunks[0].Type, ECodeChunkType::Variable);
		}
	}

	{
		FShaderSource S("template <typename T> float Fun(T x) { return (float)x; }");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: template function: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: template function: chunk type"), P.Chunks[0].Type, ECodeChunkType::Function);
		}
	}

	{
		FShaderSource S("enum EFoo { A, B = 123 };");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: enum: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: enum: chunk type"), P.Chunks[0].Type, ECodeChunkType::Enum);
		}
	}

	{
		FShaderSource S("enum class EFoo { A, B };");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: enum class: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: enum class: chunk type"), P.Chunks[0].Type, ECodeChunkType::Enum);
		}
	}

	{
		FShaderSource S("#define Foo 123");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: define: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: define: chunk type"), P.Chunks[0].Type, ECodeChunkType::Define);
		}
	}

	{
		FShaderSource S("#pragma Foo");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: pragma: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: pragma: chunk type"), P.Chunks[0].Type, ECodeChunkType::Pragma);
		}
	}

	{
		FShaderSource S("ConstantBuffer<Foo> CB : register ( b123, space456);");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: ConstantBuffer<Foo>: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: ConstantBuffer<Foo>: chunk type"), P.Chunks[0].Type, ECodeChunkType::Variable);
		}
	}

	{
		FShaderSource S("namespace NS1 { void Fun() {}; } namespace NS2 { void Fun() {}; }");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: namespaces: num chunks"), P.Chunks.Num(), 2)
		{
			TEST_IF_EQUAL(TEXT("ParseShader: namespaces: num namespaces"), P.Namespaces.Num(), 2)
			{
				TEST_EQUAL(TEXT("ParseShader: namespaces: chunk 0 namespace"), P.Chunks[0].Namespace, 0);
				TEST_EQUAL(TEXT("ParseShader: namespaces: chunk 1 namespace"), P.Chunks[1].Namespace, 1);
			}
		}
	}

	{
		FShaderSource S("template< typename T > TMyStruct<T> operator + ( TMyStruct<T> A, T B ) { /*...*/ }");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: operators: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: operators: chunk type"), P.Chunks[0].Type, ECodeChunkType::Operator);
			TEST_IF_EQUAL(TEXT("ParseShader: operators: chunk 0: num blocks"), P.Chunks[0].Blocks.Num(), 8)
			{
				FShaderSource::FStringType Args = FShaderSource::FStringType(P.Chunks[0].FindFirstBlockByType(EBlockType::Args));
				TEST_EQUAL(TEXT("ParseShader: operators: chunk 0: type name"), Args, "( TMyStruct<T> A, T B )");

				FShaderSource::FStringType TypeName = FShaderSource::FStringType(P.Chunks[0].FindFirstBlockByType(EBlockType::Type));
				TEST_EQUAL(TEXT("ParseShader: operators: chunk 0: type name"), TypeName, "TMyStruct");

				FShaderSource::FStringType OperatorName = FShaderSource::FStringType(P.Chunks[0].FindFirstBlockByType(EBlockType::OperatorName));
				TEST_EQUAL(TEXT("ParseShader: operators: chunk 0: operator name"), OperatorName, "+");
			}
		}
	}

	{
		FShaderSource S("typedef Bar Foo;");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: standard typedef: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: standard typedef: chunk type"), P.Chunks[0].Type, ECodeChunkType::Typedef);
		}
	}

	{
		FShaderSource S("typedef Bar Foo; static const Foo = Bar(0);");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: standard typedef: num chunks"), P.Chunks.Num(), 2)
		{
			TEST_EQUAL(TEXT("ParseShader: standard typedef: chunk type"), P.Chunks[0].Type, ECodeChunkType::Typedef);
			TEST_EQUAL(TEXT("ParseShader: standard typedef: chunk type"), P.Chunks[1].Type, ECodeChunkType::Variable);
		}
	}

	{
		FShaderSource S("[Shader(\"node\")] [NodeID(\"WorkgraphNodeArray\", 0)] void WorkgraphNodeArray_0() {}");
		auto P = ParseShader(S);
		TEST_IF_EQUAL(TEXT("ParseShader: workgraph node: num chunks"), P.Chunks.Num(), 1)
		{
			TEST_EQUAL(TEXT("ParseShader: workgraph node, chunk type"), P.Chunks[0].Type, ECodeChunkType::Function);
			TEST_IF_EQUAL(TEXT("ParseShader: workgraph node: chunk 0: num blocks"), P.Chunks[0].Blocks.Num(), 6)
			{
				TEST_EQUAL(TEXT("ParseShader: workgraph node: chunk 0: block 1: block type"), P.Chunks[0].Blocks[1].Type, EBlockType::NodeId);
			}
		}
	}
}

TEST_CASE_NAMED(FShaderMinifierMinifyTest, "Shaders::ShaderMinifier::Minify", "[EditorContext][EngineFilter]")
{
	using namespace UE::ShaderMinifier;

	FShaderSource TestShaderCode(
		R"(// dxc /T cs_6_6 /E MainCS MinifierTest.hlsl 
struct FFoo
{
	float X;
	float Y;
};

#pragma test_pragma
struct FBar
{
	FFoo Foo;
};

uint GUnreferencedParameter;

struct FUnreferencedStruct
{
	uint X;
};

uint UnreferencedFunction()
{
	return GUnreferencedParameter;
}

_Pragma("")

_Pragma("dxc diagnostic push")
_Pragma("dxc diagnostic ignored \"-Wall\"") _Pragma("dxc diagnostic ignored \"-Wconversion\"")
_Pragma("dxc diagnostic pop")

typedef Texture2D<float4> UnreferencedTypedef;

#define COMPILER_DEFINITION_TEST 123
float Sum(in FBar Param)
{
	return Param.Foo.X + Param.Foo.Y;
}

float FunA()
{
	// Comment inside function
	FBar Temp;
	Temp.Foo.X = 1;
	Temp.Foo.Y = 2;
	return Sum(Temp);
}

float FunB(int Param)
{
	return FunA() * (float)Param;
}

#line 1000 "MinifierTest.hlsl" //-V011
// Test comment 1
void EmptyFunction(){}

struct
{
	int Foo;
	int Bar;
} GAnonymousStruct;

struct FStructA
{
	int Foo;
	int Bar;
} GStructA;

namespace NS1 {
namespace NS2 {
static const struct FStructB
{
	int Foo;
} GStructB = {123};

static const struct FStructC
{
	int Foo;
} GStructC = { GStructA.Foo };
}} // NS1::NS2

namespace NS3 {
static const struct
{
	int Foo;
} GInitializedAnonymousStructA = { GStructA.Foo };
} // NS3

static const struct
{
	int Foo;
} GInitializedAnonymousStructB = { 123 };

typedef RWBuffer<float4> OutputBufferType;
OutputBufferType OutputBuffer;

struct FTypedefUsedStruct
{
	float Foo;
};
typedef StructuredBuffer<FTypedefUsedStruct> FTypedefUsed;
typedef FTypedefUsed FTypedefUsedChained;
typedef FTypedefUsedChained FTypedefUsedChainedUnused;
FTypedefUsedChained TypedefUsedBuffer;

struct FTypedefUnusedStruct
{
	float Foo;
};
typedef StructuredBuffer<FTypedefUnusedStruct> FTypedefUnused;
FTypedefUnused TypedefUnusedBuffer;

template<typename T> struct TUsedTemplate { T Value; };
template<typename T> TUsedTemplate<T> operator*(TUsedTemplate<T> A, T B) { return (TUsedTemplate<T>)0; }

template<typename T> struct TUnusedTemplate { T Value[2]; };
template<typename T> TUnusedTemplate<T> operator%(TUnusedTemplate<T> A, T B) { return (TUnusedTemplate<T>)0; }

enum EEnumUsed : int
{
	ENUM_USED_PART_1 = 0,
	ENUM_USED_PART_2 = 1,
};

enum EEnumUnused
{
	ENUM_UNUSED_PART_1,
	ENUM_UNUSED_PART_2,
};

struct FWorkgraphRecord
{
	int Foo;
};

// Test comment 2
[numthreads(1,1,1)]
// Comment during function declaration
void MainCS(
	NodeOutput<FWorkgraphRecord> WorkgraphNode,
	EmptyNodeOutput WorkgraphNodeAliased,
	[NodeID("WorkgraphNodeArray", 0)]
	EmptyNodeOutputArray WorkgraphNodeArray
	)
{
	using namespace NS1::NS2;
	using namespace NS3;
	TUsedTemplate Foo;
	float A = FunB(GAnonymousStruct.Foo);
	float B = FunB(GStructA.Bar + GStructB.Foo + GStructC.Foo);
	float C = FunB(GInitializedAnonymousStructA.Foo + GInitializedAnonymousStructB.Foo);
	float D = TypedefUsedBuffer[ENUM_USED_PART_1].Foo;
	OutputBuffer[0] = A + B + D;
}

[numthreads(1,1,1)]
void UnreferencedEntryPoint()
{
}

[Shader("node")]
void WorkgraphNode()
{
}

[Shader("node")]
[NodeID("WorkgraphNodeAliased")]
void WorkgraphNodeFunction()
{
}

[Shader("node")]
[NodeID("WorkgraphNodeArray", 0)]
void WorkgraphNodeArray_0()
{
}

[Shader("node")]
[NodeID("WorkgraphNodeArray", 1)]
void WorkgraphNodeArray_1()
{
}
)");

	auto ChunkPresent = [](const FParsedShader& Parsed, FShaderSource::FViewType Name)
	{
		for (const FCodeChunk& Chunk : Parsed.Chunks)
		{
			for (const FCodeBlock& Block : Chunk.Blocks)
			{
				if (Block == Name)
				{
					return true;
				}
			}
		}
		return false;
	};

	FParsedShader Parsed = ParseShader(TestShaderCode);

	{
		FDiagnostics Diagnostics;
		FShaderSource Minified = FShaderSource(MinifyShader(Parsed, "EmptyFunction", EMinifyShaderFlags::None, Diagnostics));
		FParsedShader MinifiedParsed = ParseShader(Minified);
		TEST_IF_EQUAL(TEXT("MinifyShader: EmptyFunction: num chunks"), MinifiedParsed.Chunks.Num(), 7)
		{
			TEST_EQUAL(TEXT("MinifyShader: EmptyFunction: pragma"), MinifiedParsed.Chunks[0].GetCode(), "#pragma test_pragma");
			TEST_EQUAL(TEXT("MinifyShader: EmptyFunction: C99 pragma 1"), MinifiedParsed.Chunks[1].GetCode(), "_Pragma(\"\")");
			TEST_EQUAL(TEXT("MinifyShader: EmptyFunction: C99 pragma 2"), MinifiedParsed.Chunks[2].GetCode(), "_Pragma(\"dxc diagnostic push\")");
			TEST_EQUAL(TEXT("MinifyShader: EmptyFunction: C99 pragma 3"), MinifiedParsed.Chunks[3].GetCode(), "_Pragma(\"dxc diagnostic ignored \\\"-Wall\\\"\") _Pragma(\"dxc diagnostic ignored \\\"-Wconversion\\\"\")");
			TEST_EQUAL(TEXT("MinifyShader: EmptyFunction: C99 pragma 4"), MinifiedParsed.Chunks[4].GetCode(), "_Pragma(\"dxc diagnostic pop\")");
			TEST_EQUAL(TEXT("MinifyShader: EmptyFunction: define"), MinifiedParsed.Chunks[5].GetCode(), "#define COMPILER_DEFINITION_TEST 123");
			TEST_EQUAL(TEXT("MinifyShader: EmptyFunction: function"), MinifiedParsed.Chunks[6].GetCode(), "void EmptyFunction(){}");
		}
	}

	{
		FDiagnostics Diagnostics;
		FShaderSource Minified = FShaderSource(MinifyShader(Parsed, "MainCS", EMinifyShaderFlags::OutputReasons, Diagnostics));
		FParsedShader MinifiedParsed = ParseShader(Minified);

		// Expect true:
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains MainCS"), ChunkPresent(MinifiedParsed, "MainCS"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains FFoo"), ChunkPresent(MinifiedParsed, "FFoo"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains FBar"), ChunkPresent(MinifiedParsed, "FBar"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains Sum"), ChunkPresent(MinifiedParsed, "Sum"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains FunA"), ChunkPresent(MinifiedParsed, "FunA"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains FunB"), ChunkPresent(MinifiedParsed, "FunB"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains GAnonymousStruct"), ChunkPresent(MinifiedParsed, "GAnonymousStruct"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains GStructA"), ChunkPresent(MinifiedParsed, "GStructA"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains GStructB"), ChunkPresent(MinifiedParsed, "GStructB"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains GStructC"), ChunkPresent(MinifiedParsed, "GStructC"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains GInitializedAnonymousStructA"), ChunkPresent(MinifiedParsed, "GInitializedAnonymousStructA"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains GInitializedAnonymousStructB"), ChunkPresent(MinifiedParsed, "GInitializedAnonymousStructB"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains OutputBufferType"), ChunkPresent(MinifiedParsed, "OutputBufferType"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains OutputBuffer"), ChunkPresent(MinifiedParsed, "OutputBuffer"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains FTypedefUsedStruct"), ChunkPresent(MinifiedParsed, "FTypedefUsedStruct"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains FTypedefUsed"), ChunkPresent(MinifiedParsed, "FTypedefUsed"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains FTypedefUsedChained"), ChunkPresent(MinifiedParsed, "FTypedefUsedChained"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains TypedefUsedBuffer"), ChunkPresent(MinifiedParsed, "TypedefUsedBuffer"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains struct TUnusedTemplate"), MinifiedParsed.Source.Contains("struct TUsedTemplate"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains TUsedTemplate<T> operator*"), MinifiedParsed.Source.Contains("TUsedTemplate<T> operator*"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains EEnumUsed"), MinifiedParsed.Source.Contains("EEnumUsed"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains ENUM_USED_PART_1"), MinifiedParsed.Source.Contains("ENUM_USED_PART_1"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains ENUM_USED_PART_2"), MinifiedParsed.Source.Contains("ENUM_USED_PART_2"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains struct FWorkgraphRecord"), MinifiedParsed.Source.Contains("struct FWorkgraphRecord"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains void WorkgraphNode()"), MinifiedParsed.Source.Contains("void WorkgraphNode()"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains void WorkgraphNodeFunction()"), MinifiedParsed.Source.Contains("void WorkgraphNodeFunction()"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains void WorkgraphNodeArray_0()"), MinifiedParsed.Source.Contains("void WorkgraphNodeArray_0()"));
		TEST_TRUE(TEXT("MinifyShader: MainCS: contains void WorkgraphNodeArray_1()"), MinifiedParsed.Source.Contains("void WorkgraphNodeArray_1()"));

		// Expect false:
		TEST_FALSE(TEXT("MinifyShader: MainCS: contains UnreferencedFunction"), ChunkPresent(MinifiedParsed, "UnreferencedFunction"));
		TEST_FALSE(TEXT("MinifyShader: MainCS: contains FUnreferencedStruct"), ChunkPresent(MinifiedParsed, "FUnreferencedStruct"));
		TEST_FALSE(TEXT("MinifyShader: MainCS: contains GUnreferencedParameter"), ChunkPresent(MinifiedParsed, "GUnreferencedParameter"));
		TEST_FALSE(TEXT("MinifyShader: MainCS: contains FTypedefUsedChainedUnused"), ChunkPresent(MinifiedParsed, "FTypedefUsedChainedUnused"));
		TEST_FALSE(TEXT("MinifyShader: MainCS: contains FTypedefUnusedStruct"), ChunkPresent(MinifiedParsed, "FTypedefUnusedStruct"));
		TEST_FALSE(TEXT("MinifyShader: MainCS: contains FTypedefUnused"), ChunkPresent(MinifiedParsed, "FTypedefUnused"));
		TEST_FALSE(TEXT("MinifyShader: MainCS: contains TypedefUnusedBuffer"), ChunkPresent(MinifiedParsed, "TypedefUnusedBuffer"));
		TEST_FALSE(TEXT("MinifyShader: MainCS: contains struct TUnusedTemplate"), MinifiedParsed.Source.Contains("struct TUnusedTemplate"));
		TEST_FALSE(TEXT("MinifyShader: MainCS: contains TUnusedTemplate<T> operator%"), MinifiedParsed.Source.Contains("TUnusedTemplate<T> operator%"));
		TEST_FALSE(TEXT("MinifyShader: MainCS: contains EEnumUnused"), MinifiedParsed.Source.Contains("EEnumUnused"));
		TEST_FALSE(TEXT("MinifyShader: MainCS: contains ENUM_UNUSED_PART_1"), MinifiedParsed.Source.Contains("ENUM_UNUSED_PART_1"));
		TEST_FALSE(TEXT("MinifyShader: MainCS: contains ENUM_UNUSED_PART_2"), MinifiedParsed.Source.Contains("ENUM_UNUSED_PART_2"));
		TEST_FALSE(TEXT("MinifyShader: MainCS: contains UnreferencedEntryPoint"), MinifiedParsed.Source.Contains("UnreferencedEntryPoint"));
	}
}

#endif // WITH_LOW_LEVEL_TESTS
