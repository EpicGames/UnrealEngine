// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderSDCE.h"
#include "SDCE/SDCEContext.h"

namespace UE::ShaderMinifier::SDCE
{
	void MinifyInPlace(const TConstArrayView<FStringView>& DCESymbols, FString& Code)
	{
		FMemMark Mark(FMemStack::Get());
		FContext SDCEContext(DCESymbols);

		// Parse and extract semantics
		SDCEContext.Analyze(Code);

		// Minify in-place to the existing chunks
		SDCEContext.MinifyInPlace();
	}
	
	bool MinifyInPlace(const FShaderCompilerInput& Input, const FShaderPreprocessOutput& PreprocessOutput, FString& Code)
	{
		TArray<FString, TInlineAllocator<32>> DCESymbols;

		// Extract symbols from pragmas
		PreprocessOutput.VisitDirectivesWithPrefix(ShaderMetadataPrefix,  [&DCESymbols](const FString* Value)
		{
			DCESymbols.Add(Value->RightChop(FCString::Strlen(ShaderMetadataPrefix)));
		});

		// Top environment
		FString SDCESymbolsStr;
		if (Input.Environment.GetCompileArgument(TEXT("UESHADERMETADATA_SDCE"), SDCESymbolsStr))
		{
			TArray<FString> EnvironmentSymbols;
			SDCESymbolsStr.ParseIntoArray(EnvironmentSymbols, TEXT(";"));
			DCESymbols.Append(EnvironmentSymbols);
		}

		// Shared environment
		if (IsValidRef(Input.SharedEnvironment) && Input.SharedEnvironment->GetCompileArgument(TEXT("UESHADERMETADATA_SDCE"), SDCESymbolsStr))
		{
			TArray<FString> EnvironmentSymbols;
			SDCESymbolsStr.ParseIntoArray(EnvironmentSymbols, TEXT(";"));
			DCESymbols.Append(EnvironmentSymbols);
		}

		// Nothing to minify?
		if (DCESymbols.IsEmpty())
		{
			return false;
		}

		// To views
		TArray<FStringView, TInlineAllocator<32>> DCESymbolViews;
		for (const FParseStringType& Symbol : DCESymbols)
		{
			DCESymbolViews.Add(Symbol);
		}

		MinifyInPlace(DCESymbolViews, Code);
		return true;
	}
}
