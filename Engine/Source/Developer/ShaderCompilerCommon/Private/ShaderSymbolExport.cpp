// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderSymbolExport.h"

#if WITH_ENGINE

#include "FileUtilities/ZipArchiveReader.h"
#include "FileUtilities/ZipArchiveWriter.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Hash/CityHash.h"
#include "Logging/LogMacros.h"
#include "Misc/CommandLine.h"
#include "Misc/PathViews.h"
#include "ShaderCompilerCore.h"

DECLARE_LOG_CATEGORY_CLASS(LogShaderSymbolExport, Display, Display);

static const TCHAR* ZipFileExtension = TEXT(".zip");
static const TCHAR* InfoFileExtension = TEXT(".info");

FShaderSymbolExport::FShaderSymbolExport(FName InShaderFormat)
	: ShaderFormat(InShaderFormat)
{
}

FShaderSymbolExport::~FShaderSymbolExport() = default;

static void DeleteExisting(IPlatformFile& PlatformFile, const FString& Directory, const FString& BaseLeafName, const TCHAR* Extension)
{
	TArray<FString> ExistingZips;
	PlatformFile.FindFiles(ExistingZips, *Directory, Extension);

	// To minimize over-deleting match the full name [BaseLeafName.]ext or the workers [BaseLeafName_]N.ext
	FString BaseLeafRoot = BaseLeafName + TEXT(".");
	FString BaseLeafWorker = BaseLeafName + TEXT("_");

	for (const FString& ZipFile : ExistingZips)
	{
		const FStringView Leaf = FPathViews::GetPathLeaf(ZipFile);
		if (Leaf.StartsWith(BaseLeafRoot) || Leaf.StartsWith(BaseLeafWorker))
		{
			PlatformFile.DeleteFile(*ZipFile);
		}
	}
}

static FString CreateNameAndDeleteOld(uint32 MultiprocessId, IPlatformFile& PlatformFile, const FString& ExportPath, const FString& BaseLeafName, const TCHAR* Extension)
{
	FString Name;
	if (MultiprocessId == 0)
	{
		DeleteExisting(PlatformFile, ExportPath, BaseLeafName, Extension);
		Name = FString::Printf(TEXT("%s%s"), *BaseLeafName, Extension);
	}
	else
	{
		Name = FString::Printf(TEXT("%s_%d%s"), *BaseLeafName, MultiprocessId, Extension);
	}
	return Name;
}

void FShaderSymbolExport::Initialize()
{
	const bool bSymbolsEnabled = ShouldWriteShaderSymbols(ShaderFormat);
	const bool bForceSymbols = FParse::Value(FCommandLine::Get(), TEXT("-ShaderSymbolsExport="), ExportPath);
	const bool bSymbolsInfoEnabled = ShouldGenerateShaderSymbolsInfo(ShaderFormat);

	if (bSymbolsEnabled || bForceSymbols || bSymbolsInfoEnabled)
	{
		// if no command line path is provided, look to the cvar first
		if (ExportPath.IsEmpty())
		{
			if (GetShaderSymbolPathOverride(ExportPath, ShaderFormat))
			{
				ExportPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*ExportPath);
			}
		}

		// if there was no path set via command line or the cvar, fall back to our default
		if (ExportPath.IsEmpty())
		{
			ExportPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(
				*(FPaths::ProjectSavedDir() / TEXT("ShaderSymbols") / ShaderFormat.ToString()));
		}

		// Setup ExportFileName to the default or the cvar override
		ExportFileName = TEXT("ShaderSymbols");
		GetShaderFileNameOverride(ExportFileName, TEXT("r.Shaders.SymbolFileNameOverride"), ShaderFormat, ShaderFormat);

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		bExportShaderSymbols = PlatformFile.CreateDirectoryTree(*ExportPath);

		if (!bExportShaderSymbols)
		{
			UE_LOGF(LogShaderSymbolExport, Error, "Failed to create shader symbols output directory. Shader symbol export will be disabled.");
		}
		else
		{
			// setup multiproc data in case we need it
			uint32 MultiprocessId = UE::GetMultiprocessId();
			bMultiprocessOwner = MultiprocessId == 0;

			// Check if the export mode is to an uncompressed/compressed archive or loose files.
			const EWriteShaderSymbols WriteShaderSymbolsOptions = GetWriteShaderSymbolsOptions(ShaderFormat);
			const bool bExportAsZip = WriteShaderSymbolsOptions != EWriteShaderSymbols::Disable;

			if (bSymbolsEnabled && (bExportAsZip || FParse::Param(FCommandLine::Get(), TEXT("ShaderSymbolsExportZip"))))
			{
				FString LeafName = CreateNameAndDeleteOld(MultiprocessId, PlatformFile, ExportPath, ExportFileName, ZipFileExtension);
				FString SingleFilePath = ExportPath / LeafName;

				IFileHandle* OutputZipFile = PlatformFile.OpenWrite(*SingleFilePath);
				if (!OutputZipFile)
				{
					UE_LOGF(LogShaderSymbolExport, Error, "Failed to create shader symbols output file \"%ls\". Shader symbol export will be disabled.", *SingleFilePath);
					bExportShaderSymbols = false;
				}
				else
				{
					// only compress the first zip file. Other ones are going to be compress in the merging
					EZipArchiveOptions ZipOptions = (WriteShaderSymbolsOptions == EWriteShaderSymbols::Compress && bMultiprocessOwner) ? EZipArchiveOptions::Deflate : EZipArchiveOptions::None;
					ZipOptions |= EZipArchiveOptions::RemoveDuplicate;
					ZipWriter = MakeUnique<FZipArchiveWriter>(OutputZipFile, ZipOptions);
				}
			}
			
			if (bSymbolsInfoEnabled)
			{
				// if we are exporting collated shader pdb info into one file
				FString LeafName = CreateNameAndDeleteOld(MultiprocessId, PlatformFile, ExportPath, ExportFileName, InfoFileExtension);
				InfoFilePath = ExportPath / LeafName;
			}
		}
	}

	if (bExportShaderSymbols)
	{
		UE_LOGF(LogShaderSymbolExport, Display, "Shader symbol export enabled. Output directory: \"%ls\"", *ExportPath);
		if (ZipWriter)
		{
			UE_LOGF(LogShaderSymbolExport, Display, "Shader symbol zip mode enabled. Shader symbols will be archived in a single (uncompressed) zip file.");
		}
	}
}

void FShaderSymbolExport::WriteSymbolData(const FString& Filename, const FString& DebugData, TConstArrayView<uint8> Contents)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderSymbolExport::WriteSymbolData);

	// No writing is possible if the Filename is empty
	if (Filename.IsEmpty())
	{
		return;
	}

	// If writing to a zipfile (for simplicity's sake, since zipwriter doesn't allow removing/replacing files), or when exporting
	// symbols info instead of/in addition to full symbols, we just always write the first copy of symbols encountered.
	// Note this means the actual symbols written in these cases may be non-deterministic, but this won't cause any real problems.
	// Otherwise, we add an additional condition based on the hash of the symbols, if the new hash is less than the one previously
	// written we re-write (so we always export a deterministic copy of the symbols in this case).

	bool bDeterministicSymbols = ZipWriter == nullptr && !ShouldGenerateShaderSymbolsInfo(ShaderFormat);

	// calculate the hash of the new symbols only if we're not exporting deterministically
	// (we don't need it in that case as described in the comments above)
	uint64 NewSymbolHash = bDeterministicSymbols ? CityHash64(reinterpret_cast<const char*>(Contents.GetData()), Contents.Num()) : 0u;

	// Read phase: early-out for shaders already exported, without acquiring the write lock.
	{
		FRWScopeLock ReadLock(SymbolWriteLock, SLT_ReadOnly);
		if (ExportedShaders.Contains(Filename))
		{
			if (!bDeterministicSymbols)
			{
				// DuplicateSymbols is only incremented in the read phase on early return, because the write phase will also increment.
				// Remember that this block may fail to detect a duplicate if two threads race to insert.
				DuplicateSymbols.fetch_add(1, std::memory_order_relaxed);
				return;
			}

			// when ensuring determinism, we further store the hash of the symbols we wrote out, and if the new hash is less than
			// the previously written one we write it again, so we always choose the same copy of symbols to write during i.e. a cook
			const FSymbolFileInfo* PrevExportedSymbolInfo = ExportedSymbolInfo.Find(Filename);
			check(PrevExportedSymbolInfo);
			check(NewSymbolHash != 0u);
			if (NewSymbolHash >= PrevExportedSymbolInfo->Hash)
			{
				DuplicateSymbols.fetch_add(1, std::memory_order_relaxed);
				return;
			}
			// Hash is lower; fall through to write phase to overwrite
		}
	}

	// Write phase: first insertion, or deterministic overwrite with a lower hash.
	// Re-check after acquiring the write lock to handle races on first insertion.
	bool bShouldWriteSymbols = ShouldWriteShaderSymbols(ShaderFormat) && Contents.Num();
	bool bShouldWriteToZip = ZipWriter != nullptr;
	{
		FRWScopeLock WriteLock(SymbolWriteLock, SLT_Write);

		bool bAlreadyInSet = false; // Two threads may be trying to insert concurrently
		ExportedShaders.Add(Filename, &bAlreadyInSet);

		if (bAlreadyInSet)
		{
			// We've already exported a copy of symbols for this shader hash and are not ensuring exports are deterministic
			DuplicateSymbols.fetch_add(1, std::memory_order_relaxed);
			if (!bDeterministicSymbols)
			{
				return;
			}

			FSymbolFileInfo* PrevExportedSymbolInfo = ExportedSymbolInfo.Find(Filename);
			check(PrevExportedSymbolInfo);
			check(NewSymbolHash != 0u);
			if (NewSymbolHash >= PrevExportedSymbolInfo->Hash)
			{
				return;
			}

			// new hash is less; adjust stats to account for file we will replace below
			TotalSymbolDataBytes -= PrevExportedSymbolInfo->Size;
			TotalSymbolData--;
		}

		if (bDeterministicSymbols)
		{
			ExportedSymbolInfo.Add(Filename, { NewSymbolHash, Contents.Num() });
		}

		// Emit periodic log messages detailing the size of the shader symbols output file/directory.
		static uint64 LastReport = 0;
		TotalSymbolDataBytes += Contents.Num();
		TotalSymbolData++;

		if ((TotalSymbolDataBytes - LastReport) >= (64 * 1024 * 1024))
		{
			UE_LOGF(LogShaderSymbolExport, Display, "Shader symbols export size: %.2f MB, count: %llu",
				double(TotalSymbolDataBytes) / (1024.0 * 1024.0), TotalSymbolData);
			LastReport = TotalSymbolDataBytes;
		}

		if (ShouldGenerateShaderSymbolsInfo(ShaderFormat) && !DebugData.IsEmpty())
		{
			// Collect the simple shader symbol information
			FString FilenameNoExt = FPaths::GetBaseFilename(Filename);
			ShaderInfos.Add({ FilenameNoExt, DebugData });
		}

		if (bShouldWriteSymbols && bShouldWriteToZip)
		{
			// Append the platform data to the zip file
			ZipWriter->AddFile(Filename, Contents, FDateTime::Now());
		}
	}
	// Outside of lock scope
	if (bShouldWriteSymbols && !bShouldWriteToZip)
	{
		// In deterministic mode, files may be overwritten.
		// If a thread wants to overwrite a file while the previous is still being written,
		// we need to wait for the first one to finish.
		WriteFile(Filename, Contents, bDeterministicSymbols);
	}
}

void FShaderSymbolExport::WriteFile(const FString& Filename, const TConstArrayView<uint8>& Contents, bool bAllowRetry)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FShaderSymbolExport::WriteFile);
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	// Write the symbols to the export directory
	const FString OutputPath = ExportPath / Filename;
	const FString Directory = FPaths::GetPath(OutputPath);

	// Filename could contain extra folders, so we need to make sure they exist first.
	if (!PlatformFile.CreateDirectoryTree(*Directory))
	{
		UE_LOGF(LogShaderSymbolExport, Error, "Failed to create shader symbol directory \"%ls\".", *Directory);
	}
	else
	{
		FFileOpenResult FileResult = PlatformFile.OpenWrite(*OutputPath, IPlatformFile::EOpenWriteFlags::None);

		const int MaxRetries = 10;
		int Retries = 0;
		while (FileResult.HasError() && bAllowRetry && (Retries++ < MaxRetries))
		{
			FPlatformProcess::Sleep(0.0f);
			FRWScopeLock WriteLock(SymbolWriteLock, SLT_Write);
			FileResult = PlatformFile.OpenWrite(*OutputPath, IPlatformFile::EOpenWriteFlags::None);
		}

		if (FileResult.HasError())
		{
			FFileSystemError& Error = FileResult.GetError();
			FString ErrorMessage = Error.GetMessage();
			UE_LOGF(LogShaderSymbolExport, Error, "Failed to open shader symbols file \"%ls\":", *OutputPath);
			UE_LOGF(LogShaderSymbolExport, Error, "%ls", *ErrorMessage);
			return;
		}

		TUniquePtr<IFileHandle> File = FileResult.StealValue();
		if (!File->Write(Contents.GetData(), Contents.Num()))
		{
			UE_LOGF(LogShaderSymbolExport, Error, "Failed to write shader symbols \"%ls\".", *OutputPath);
		}
	}
}

void FShaderSymbolExport::NotifyShaderCompilersShutdown()
{
	// Wait for all in-flight write tasks to complete before finalizing artifacts.
	AllTasksDoneEvent->Reset();
	if (PendingTaskCount.load(std::memory_order_acquire) > 0)
	{
		AllTasksDoneEvent->Wait();
	}

	// Used to match worker files, for example (if not overridden): [ShaderSymbols_]N.ext
	const FString WorkerName = ExportFileName + TEXT("_");

	if (ShaderInfos.Num())
	{
		if (InfoFilePath.Len())
		{
			IFileManager& FileManager = IFileManager::Get();
			if (bMultiprocessOwner)
			{
				// If we are the multiprocess owner merge in any other files we find which have the same FileName, outside of multiproc numbering
				// We do not want to delete other unrelated files in the export path, potentially from other cook jobs
				// We will chunk up the worker files into {Hash, Data} pairs, dedupe them with ours, and sort them all
				IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
				TArray<FString> FilesToMergeIn;
				PlatformFile.FindFiles(FilesToMergeIn, *ExportPath, InfoFileExtension);
				FilesToMergeIn.RemoveAll([&WorkerName](const FString& FileName)
					{
						return !FPathViews::GetPathLeaf(FileName).StartsWith(WorkerName);
					});

				for (const FString& InfoFile : FilesToMergeIn)
				{
					TUniquePtr<FArchive> Reader = TUniquePtr<FArchive>(FileManager.CreateFileReader(*InfoFile));
					if (Reader.IsValid())
					{
						int64 Size = Reader->TotalSize();
						TArray<uint8> RawData;
						RawData.AddUninitialized(Size);
						Reader->Serialize(RawData.GetData(), Size);
						Reader->Close();

						TArray<FString> Lines;
						FString(StringCast<TCHAR>(reinterpret_cast<const ANSICHAR*>(RawData.GetData())).Get()).ParseIntoArrayLines(Lines);

						for (const FString& Line : Lines)
						{
							int32 Space;
							Line.FindChar(TEXT(' '), Space);
							if (Space != INDEX_NONE)
							{
								FString Filename = Line.Left(Space);

								// if this symbol is new to the multiproc owner, store it
								bool bAlreadyInSet = false;
								ExportedShaders.Add(Filename, &bAlreadyInSet);
								if (bAlreadyInSet)
								{
									// The multiproc owner has already seen this hash
									DuplicateSymbols++;
								}
								else
								{
									FString DebugData = Line.Right(Line.Len() - Space - 1);
									ShaderInfos.Add({ Filename, DebugData });
								}
							}
						}
					}
					PlatformFile.DeleteFile(*InfoFile);
				}
			}

			// sort and combine the data for output
			ShaderInfos.KeySort([](const FString& A, const FString& B) { return A < B; });

			TArray<uint8> Output;
			for (TPair<FString, FString> Info : ShaderInfos)
			{
				auto TmpHash = StringCast<ANSICHAR>(*Info.Key);
				auto TmpData = StringCast<ANSICHAR>(*Info.Value);
				Output.Append((const uint8*)TmpHash.Get(), TmpHash.Length());
				Output.Add(' ');
				Output.Append((const uint8*)TmpData.Get(), TmpData.Length());
				Output.Add('\n');
			}

			TUniquePtr<FArchive> Writer = TUniquePtr<FArchive>(FileManager.CreateFileWriter(*InfoFilePath));
			if (Writer.IsValid())
			{
				Writer->Serialize(Output.GetData(), Output.Num());
				Writer->Close();
				UE_LOGF(LogShaderSymbolExport, Display, "Wrote %d records into shader symbols info output file \"%ls\".", ShaderInfos.Num(), *InfoFilePath);
				const uint32 NumDuplicates = DuplicateSymbols.load(std::memory_order_relaxed);
				const uint32 TotalShaders = ShaderInfos.Num() + NumDuplicates;
				UE_LOGF(LogShaderSymbolExport, Display, "%d total shaders, %d shaders after platform deduplication. %d duplicates (%4.1f%%).",
					TotalShaders, TotalShaders - NumDuplicates, NumDuplicates, (float)NumDuplicates / TotalShaders * 100.0f);
			}
			else
			{
				UE_LOGF(LogShaderSymbolExport, Error, "Failed to create shader symbols output file \"%ls\".", *InfoFilePath);
			}
		}
	}

	if (ZipWriter && bMultiprocessOwner)
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		TArray<FString> ZipsToMergeIn;
		PlatformFile.FindFiles(ZipsToMergeIn, *ExportPath, ZipFileExtension);
		ZipsToMergeIn.RemoveAll([&WorkerName](const FString& FileName)
			{
				return !FPathViews::GetPathLeaf(FileName).StartsWith(WorkerName);
			});

#if WITH_EDITOR // FZipArchiveReader is only available in editor
		for (const FString& ZipFile : ZipsToMergeIn)
		{
			{
				FZipArchiveReader Reader(PlatformFile.OpenRead(*ZipFile));
				bool bAllValid = false;
				if (Reader.IsValid())
				{
					bAllValid = true;
					for (const FString& EmbeddedFileName : Reader.GetFileNames())
					{
						TArray<uint8> Contents;
						if (!Reader.TryReadFile(EmbeddedFileName, Contents))
						{
							bAllValid = false;
							continue;
						}
						ZipWriter->AddFile(EmbeddedFileName, Contents, FDateTime::Now());
					}
				}
				if (!bAllValid)
				{
					UE_LOGF(LogShaderSymbolExport, Error,
						"Failed to read from CookWorker shader symbols output file \"%ls\". Some shader symbols will be missing.",
						*ZipFile);
				}
			}
			PlatformFile.DeleteFile(*ZipFile);
		}
#else
		UE_CLOGF(!ZipsToMergeIn.IsEmpty(), LogShaderSymbolExport, Error,
			"Cannot merge zips from multiprocess instances in %ls; merging is only available in editor.", *ExportPath);
#endif
	}
	ZipWriter.Reset();
}

#endif // WITH_ENGINE
