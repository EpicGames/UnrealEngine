// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpirvCommon.h"

SHADERCOMPILERCOMMON_API const TCHAR* SpirvBuiltinToString(const SpvBuiltIn BuiltIn)
{
	switch (BuiltIn)
	{
	case SpvBuiltInPosition:					return TEXT("gl_Position");
	case SpvBuiltInPointSize:					return TEXT("gl_PointSize");
	case SpvBuiltInClipDistance:				return TEXT("gl_ClipDistance");
	case SpvBuiltInCullDistance:				return TEXT("gl_CullDistance");
	case SpvBuiltInVertexId:					return TEXT("gl_VertexID");
	case SpvBuiltInInstanceId:					return TEXT("gl_InstanceID");
	case SpvBuiltInPrimitiveId:					return TEXT("gl_PrimitiveID");
	case SpvBuiltInInvocationId:				return TEXT("gl_InvocationID");
	case SpvBuiltInLayer:						return TEXT("gl_Layer");
	case SpvBuiltInViewportIndex:				return TEXT("gl_ViewportIndex");
	case SpvBuiltInTessLevelOuter:				return TEXT("gl_TessLevelOuter");
	case SpvBuiltInTessLevelInner:				return TEXT("gl_TessLevelInner");
	case SpvBuiltInTessCoord:					return TEXT("gl_TessCoord");
	case SpvBuiltInPatchVertices:				return TEXT("gl_PatchVertices");
	case SpvBuiltInFragCoord:					return TEXT("gl_FragCoord");
	case SpvBuiltInPointCoord:					return TEXT("gl_PointCoord");
	case SpvBuiltInFrontFacing:					return TEXT("gl_FrontFacing");
	case SpvBuiltInSampleId:					return TEXT("gl_SampleID");
	case SpvBuiltInSamplePosition:				return TEXT("gl_SamplePosition");
	case SpvBuiltInSampleMask:					return TEXT("gl_SampleMask");
	case SpvBuiltInFragDepth:					return TEXT("gl_FragDepth");
	case SpvBuiltInHelperInvocation:			return TEXT("gl_HelperInvocation");
	case SpvBuiltInNumWorkgroups:				return TEXT("gl_NumWorkgroups");
	case SpvBuiltInWorkgroupSize:				return TEXT("gl_WorkgroupSize");
	case SpvBuiltInWorkgroupId:					return TEXT("gl_WorkgroupID");
	case SpvBuiltInLocalInvocationId:			return TEXT("gl_LocalInvocationID");
	case SpvBuiltInGlobalInvocationId:			return TEXT("gl_GlobalInvocationID");
	case SpvBuiltInLocalInvocationIndex:		return TEXT("gl_LocalInvocationIndex");
	case SpvBuiltInWorkDim:						return TEXT("gl_WorkDim");
	case SpvBuiltInGlobalSize:					return TEXT("gl_GlobalSize");
	case SpvBuiltInEnqueuedWorkgroupSize:		return TEXT("gl_EnqueuedWorkgroupSize");
	case SpvBuiltInGlobalOffset:				return TEXT("gl_GlobalOffset");
	case SpvBuiltInGlobalLinearId:				return TEXT("gl_GlobalLinearID");
	case SpvBuiltInSubgroupSize:				return TEXT("gl_SubgroupSize");
	case SpvBuiltInSubgroupMaxSize:				return TEXT("gl_SubgroupMaxSize");
	case SpvBuiltInNumSubgroups:				return TEXT("gl_NumSubgroups");
	case SpvBuiltInNumEnqueuedSubgroups:		return TEXT("gl_NumEnqueuedSubgroups");
	case SpvBuiltInSubgroupId:					return TEXT("gl_SubgroupID");
	case SpvBuiltInSubgroupLocalInvocationId:	return TEXT("gl_SubgroupLocalInvocationID");
	case SpvBuiltInVertexIndex:					return TEXT("gl_VertexIndex");
	case SpvBuiltInInstanceIndex:				return TEXT("gl_InstanceIndex");
	case SpvBuiltInSubgroupEqMask:				return TEXT("gl_SubgroupEqMask");
	case SpvBuiltInSubgroupGeMask:				return TEXT("gl_SubgroupGeMask");
	case SpvBuiltInSubgroupGtMask:				return TEXT("gl_SubgroupGtMask");
	case SpvBuiltInSubgroupLeMask:				return TEXT("gl_SubgroupLeMask");
	case SpvBuiltInSubgroupLtMask:				return TEXT("gl_SubgroupLtMask");
	case SpvBuiltInBaseVertex:					return TEXT("gl_BaseVertex");
	case SpvBuiltInBaseInstance:				return TEXT("gl_BaseInstance");
	case SpvBuiltInDrawIndex:					return TEXT("gl_DrawIndex");
	case SpvBuiltInDeviceIndex:					return TEXT("gl_DeviceIndex");
	case SpvBuiltInViewIndex:					return TEXT("gl_ViewIndex");

	// Ray tracing
	case SpvBuiltInLaunchIdKHR:					return TEXT("gl_LaunchIDEXT");
	case SpvBuiltInLaunchSizeKHR:				return TEXT("gl_LaunchSizeEXT");
	case SpvBuiltInInstanceCustomIndexKHR:		return TEXT("gl_InstanceCustomIndexEXT");
	case SpvBuiltInRayGeometryIndexKHR:			return TEXT("gl_GeometryIndexEXT");
	case SpvBuiltInWorldRayOriginKHR:			return TEXT("gl_WorldRayOriginEXT");
	case SpvBuiltInWorldRayDirectionKHR:		return TEXT("gl_WorldRayDirectionEXT");
	case SpvBuiltInObjectRayOriginKHR:			return TEXT("gl_ObjectRayOriginEXT");
	case SpvBuiltInObjectRayDirectionKHR:		return TEXT("gl_ObjectRayDirectionEXT");
	case SpvBuiltInRayTminKHR:					return TEXT("gl_RayTminEXT");
	case SpvBuiltInRayTmaxKHR:					return TEXT("gl_RayTmaxEXT");
	case SpvBuiltInIncomingRayFlagsKHR:			return TEXT("gl_IncomingRayFlagsEXT");
	case SpvBuiltInHitKindKHR:					return TEXT("gl_HitKindEXT");
	case SpvBuiltInObjectToWorldKHR:			return TEXT("gl_ObjectToWorldEXT");
	case SpvBuiltInWorldToObjectKHR:			return TEXT("gl_WorldToObjectEXT");
	
	// SPV_NV_mesh_shader
	case SpvBuiltInPrimitiveIndicesNV:			return TEXT("gl_PrimitiveIndicesNV");
	case SpvBuiltInPrimitiveCountNV:			return TEXT("gl_PrimitiveCountNV");

	// SPV_EXT_mesh_shader
	case SpvBuiltInPrimitiveTriangleIndicesEXT:	return TEXT("gl_PrimitiveTriangleIndicesEXT");
	}
	return nullptr;
}

static int32 FindOffsetToFirstSpirvEntryPoint(const FSpirv& Spirv)
{
	// Iterate over all SPIR-V instructions until we have what we need
	for (FSpirvConstIterator Iter = Spirv.begin(); Iter != Spirv.end(); ++Iter)
	{
		switch (Iter.Opcode())
		{
		case SpvOpEntryPoint:
		{
			// Check if we found our entry point.
			// With RayTracing, there can be multiple entry point declarations in a single SPIR-V module.
			const ANSICHAR* Name = Iter.OperandAsString(3);
			// Return word offset to OpEntryPoint instruction
			return (int32)Spirv.GetWordOffset(Iter, 3);
		}
		}
	}

	return -1;
}

SHADERCOMPILERCOMMON_API void FindOffsetToSpirvEntryPoint(const FSpirv& Spirv, const ANSICHAR* EntryPointName, uint32& OutWordOffsetToEntryPoint, uint32& OutWordOffsetToMainName)
{
	// Iterate over all SPIR-V instructions until we have what we need
	for (FSpirvConstIterator Iter = Spirv.begin(); Iter != Spirv.end(); ++Iter)
	{
		switch (Iter.Opcode())
		{
		case SpvOpEntryPoint:
		{
			// Check if we found our entry point.
			// With RayTracing, there can be multiple entry point declarations in a single SPIR-V module.
			const ANSICHAR* Name = Iter.OperandAsString(3);
			if (FCStringAnsi::Strcmp(Name, EntryPointName) == 0)
			{
				// Return word offset to OpEntryPoint instruction
				check(OutWordOffsetToEntryPoint == 0);
				OutWordOffsetToEntryPoint = Spirv.GetWordOffset(Iter, 3);
			}
		}
		break;

		case SpvOpName:
		{
			const ANSICHAR* Name = Iter.OperandAsString(2);
			if (FCStringAnsi::Strcmp(Name, EntryPointName) == 0)
			{
				// Return word offset to OpName instruction that refers to the main entry point
				check(OutWordOffsetToMainName == 0);
				OutWordOffsetToMainName = Spirv.GetWordOffset(Iter, 2);
			}
		}
		break;

		case SpvOpDecorate:
		case SpvOpMemberDecorate:
		case SpvOpFunction:
		{
			// With the first annotation, type declaration, or function declaration,
			// there can't be any more entry point or debug instructions (i.e. OpEntryPoint and OpName).
			// However, only the OpFunction is guaranteed to appear.
			return;
		}
		}
	}
}

static const ANSICHAR* GSpirvPlaceholderEntryPointName = "main_00000000_00000000";

static void RenameFixedSizeSpirvString(FSpirv& Spirv, uint32 WordOffsetToString, uint32 CRC)
{
	char* TargetString = reinterpret_cast<char*>(Spirv.Data.GetData() + WordOffsetToString);
	check(!FCStringAnsi::Strcmp(TargetString, GSpirvPlaceholderEntryPointName));
	const uint32 SpirvByteSize = static_cast<uint32>(Spirv.GetByteSize());
	FCStringAnsi::Sprintf(TargetString, "main_%0.8x_%0.8x", SpirvByteSize, CRC);
};


SHADERCOMPILERCOMMON_API const ANSICHAR* FindSpirvEntryPoint(const FSpirv& Spirv)
{
	int32 OffsetToEntryPoint = FindOffsetToFirstSpirvEntryPoint(Spirv);
	if (OffsetToEntryPoint == -1)
	{
		return nullptr;
	}
	return reinterpret_cast<const ANSICHAR*>(Spirv.Data.GetData() + OffsetToEntryPoint);
}

SHADERCOMPILERCOMMON_API const ANSICHAR* PatchSpirvEntryPointWithCRC(FSpirv& Spirv, uint32& OutCRC)
{
	// Find offsets to entry point strings and generate CRC over the module
	uint32 OffsetToEntryPoint = 0, OffsetToMainName = 0;
	FindOffsetToSpirvEntryPoint(Spirv, GSpirvPlaceholderEntryPointName, OffsetToEntryPoint, OffsetToMainName);
	OutCRC = FCrc::MemCrc32(Spirv.GetByteData(), Spirv.GetByteSize());

	// Patch the (optional) entry point name decoration; this can be stripped out by some optimization passes
	RenameFixedSizeSpirvString(Spirv, OffsetToEntryPoint, OutCRC);
	if (OffsetToMainName != 0)
	{
		RenameFixedSizeSpirvString(Spirv, OffsetToMainName, OutCRC);
	}

	return reinterpret_cast<const ANSICHAR*>(Spirv.Data.GetData() + OffsetToEntryPoint);
}

struct FSpirvVariableIdAndName
{
	uint32 Id;
	const ANSICHAR* AnsiName;
	const TCHAR* WideName;
};

static void ParseSpirvVariableIDs(const FSpirv& Spirv, SpvStorageClass StorageClass, TArray<FSpirvVariableIdAndName>& OutVariables)
{
	for (FSpirvConstIterator Iter = Spirv.begin(); Iter != Spirv.end(); ++Iter)
	{
		switch (Iter.Opcode())
		{
		case SpvOpVariable:
		{
			const SpvStorageClass VariableStorageClass = Iter.OperandAs<SpvStorageClass>(3);
			if (VariableStorageClass == StorageClass)
			{
				const uint32 VariableID = Iter.Operand(2);
				OutVariables.Add({ VariableID, nullptr });
			}
		}
		break;

		case SpvOpFunction:
		{
			// Early exit with first function declaration
			return;
		}
		}
	}
}

static void ParseSpirvVariableNames(const FSpirv& Spirv, TArray<FSpirvVariableIdAndName>& InOutVariables)
{
	for (FSpirvConstIterator Iter = Spirv.begin(); Iter != Spirv.end(); ++Iter)
	{
		switch (Iter.Opcode())
		{
		case SpvOpName:
		{
			// Iterate through all variables - there shouldn't be a lot of them when searching for a specific storage class (somewhere in the range 1-32).
			const uint32 Target = Iter.Operand(1);
			for (FSpirvVariableIdAndName& Variable : InOutVariables)
			{
				if (Variable.Id == Target)
				{
					Variable.AnsiName = Iter.OperandAsString(2);
					break;
				}
			}
		}
		break;

		case SpvOpDecorate:
		{
			// Is this a decoration for a built-in variable?
			const SpvDecoration Decoration = Iter.OperandAs<SpvDecoration>(2);
			if (Decoration == SpvDecorationBuiltIn)
			{
				const uint32 Target = Iter.Operand(1);
				for (FSpirvVariableIdAndName& Variable : InOutVariables)
				{
					if (Variable.Id == Target)
					{
						const SpvBuiltIn BuiltIn = Iter.OperandAs<SpvBuiltIn>(3);
						Variable.WideName = SpirvBuiltinToString(BuiltIn);
						break;
					}
				}
			}
		}
		break;

		case SpvOpFunction:
		{
			// Early exit with first function declaration
			return;
		}
		}
	}
}

// Replaces '.' characters from the specified SPIR-V name with '_'.
static FString SanitizeSpirvName(const ANSICHAR* Name)
{
	FString NameStr = ANSI_TO_TCHAR(Name);
	for (TCHAR& Chr : NameStr)
	{
		if (Chr == TEXT('.'))
		{
			Chr = TEXT('_');
		}
	}
	return NameStr;
}

SHADERCOMPILERCOMMON_API void ParseSpirvGlobalVariables(const FSpirv& Spirv, SpvStorageClass StorageClass, TArray<FString>& OutVariableNames)
{
	checkf(StorageClass != SpvStorageClassFunction, TEXT("StorageClass must not be SpvStorageClassFunction"));

	// Parse variable IDs and names from SPIR-V module
	TArray<FSpirvVariableIdAndName> IntermediateVariables;
	ParseSpirvVariableIDs(Spirv, StorageClass, IntermediateVariables);

	// Now find names for variable IDs, since OpName comes first in SPIR-V module but assigns their values to target variables that are declared later
	ParseSpirvVariableNames(Spirv, IntermediateVariables);

	// Convert variable names to FString output container
	OutVariableNames.Reserve(IntermediateVariables.Num());
	for (const FSpirvVariableIdAndName& Variable : IntermediateVariables)
	{
		if (const ANSICHAR* AnsiName = Variable.AnsiName)
		{
			// Names in SPIR-V module may contain invalid characters for high-level source languages, e.g. "in.var.TEXCOORD0"
			OutVariableNames.Add(SanitizeSpirvName(AnsiName));
		}
		else if (const TCHAR* WideName = Variable.WideName)
		{
			// No sanitization of this name is necessary; this is supposed to come from SpirvBuiltinToString().
			OutVariableNames.Add(WideName);
		}
		else
		{
			checkf(0, TEXT("FSpirvVariableIdAndName entry with invalid name fields (ID = %u)"), Variable.Id);
		}
	}
}

// Scan the early sections of the SPIRV to find the used members of the specified structure
SHADERCOMPILERCOMMON_API TArray<FSpirvStructureMemberData> ParseSpirvStructureMembers(const FSpirv& Spirv, FAnsiStringView StructureName)
{
	uint32_t StructureId = MAX_uint32;

	TArray<FSpirvStructureMemberData> Members;

	TMap<uint32, uint32> TypeSizes;

	bool bReachedBody = false;
	for (FSpirvConstIterator Iter = Spirv.begin(); Iter != Spirv.end() && !bReachedBody; ++Iter)
	{
		switch (Iter.Opcode())
		{

		// Use OpName to find the TypeId of the two structs we're interested in
		case SpvOpName:
		{
			// Operands:
			// 1 - Result Id
			// 2 - Name

			if (StructureId == MAX_uint32)
			{
				FAnsiStringView NameStr = FAnsiStringView(Iter.OperandAsString(2));
				if (NameStr.EndsWith(StructureName))
				{
					StructureId = Iter.Operand(1);
				}
			}

			break;
		}

		// Grab the OpMemberName for each member of the loose globals
		case SpvOpMemberName:
		{
			// Operands:
			// 1 - Type Id
			// 2 - Member (index)
			// 3 - Name

			if (Iter.Operand(1) == StructureId)
			{
				const uint32_t Index = Iter.Operand(2);
				if (Index >= (uint32)Members.Num())
				{
					Members.SetNum(Index + 1);
				}
				Members[Index].Name = FString(Iter.OperandAsString(3));
			}
			break;
		}

		// Take down the offset of each member (unused members aren't decorated)
		case SpvOpMemberDecorate:
		{
			// Operands:
			// 1 - Type Id
			// 2 - Member (index)
			// 3 - Decoration
			// 4 - Decoration Value

			if (Iter.Operand(1) == StructureId)
			{
				if (Iter.Operand(3) == SpvDecorationOffset)
				{
					const uint32_t Index = Iter.Operand(2);
					Members[Index].Offset = Iter.Operand(4);
				}
			}
			break;
		}

		case SpvOpTypeInt:
		case SpvOpTypeFloat:
		{
			// Operands:
			// 1 - Type Id
			// 2 - Bit width
			// 3 - Signedness (for ints)

			TypeSizes.Add(Iter.Operand(1), Iter.Operand(2) / 8);
			break;
		}

		case SpvOpTypeVector:
		{
			// Operands:
			// 1 - Type Id
			// 2 - Component Type Id
			// 3 - Component Count

			if (TypeSizes.Contains(Iter.Operand(1)))
			{
				const uint32 ComponentSize = TypeSizes[Iter.Operand(2)];
				TypeSizes.Add(Iter.Operand(1), Iter.Operand(3) * ComponentSize);
			}
			break;
		}

		case SpvOpTypeStruct:
		{
			// Operands:
			// 1 - Type Id
			// 2,3,n - Member 0,1,n Type

			if (Iter.Operand(1) == StructureId)
			{
				const uint32 OperandStartIndex = 2;
				for (uint32 OperandIndex = OperandStartIndex; OperandIndex < Iter.WordCount(); ++OperandIndex)
				{
					Members[OperandIndex - OperandStartIndex].Size = TypeSizes[Iter.Operand(OperandIndex)];
				}
			}
			break;
		}

		// When the body starts, that means there's no more useful information for us
		case SpvOpFunction:
			bReachedBody = true;
			break;
		}
	}

	return Members;
}