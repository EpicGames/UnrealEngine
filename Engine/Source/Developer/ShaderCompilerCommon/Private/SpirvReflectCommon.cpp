// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpirvReflectCommon.h"

void FSpirvReflectBindings::AddDescriptorBinding(SpvReflectDescriptorBinding* InBinding)
{
	check(InBinding != nullptr);
	switch (InBinding->resource_type)
	{
	case SPV_REFLECT_RESOURCE_FLAG_SAMPLER:
		// Gather sampler states (i.e. SamplerState or SamplerComparisonState)
		check(InBinding->descriptor_type == SPV_REFLECT_DESCRIPTOR_TYPE_SAMPLER);
		if (InBinding->accessed)
		{
			Samplers.Add(InBinding);
		}
		break;

	case SPV_REFLECT_RESOURCE_FLAG_CBV:
		// Gather constant buffers (i.e. cbuffer or ConstantBuffer<T>).
		check(InBinding->descriptor_type == SPV_REFLECT_DESCRIPTOR_TYPE_UNIFORM_BUFFER);
		if (InBinding->accessed)
		{
			UniformBuffers.Add(InBinding);
		}
		break;

	case SPV_REFLECT_RESOURCE_FLAG_SRV:
		// Gather SRV resources (e.g. Buffer, StructuredBuffer, Texture2D etc.)
		switch (InBinding->descriptor_type)
		{
			/*case SPV_REFLECT_DESCRIPTOR_TYPE_COMBINED_IMAGE_SAMPLER:
			{
				if (InBinding->accessed)
				{
					Samplers.Add(InBinding);
				}
				break;
			}*/
		case SPV_REFLECT_DESCRIPTOR_TYPE_SAMPLED_IMAGE:
			if (InBinding->accessed)
			{
				TextureSRVs.Add(InBinding);
			}
			break;

		case SPV_REFLECT_DESCRIPTOR_TYPE_UNIFORM_TEXEL_BUFFER:
			if (InBinding->accessed)
			{
				TBufferSRVs.Add(InBinding);
			}
			break;

		case SPV_REFLECT_DESCRIPTOR_TYPE_STORAGE_BUFFER:
			if (InBinding->accessed)
			{
				SBufferSRVs.Add(InBinding);
			}
			break;

		case SPV_REFLECT_DESCRIPTOR_TYPE_ACCELERATION_STRUCTURE_KHR:
			if (InBinding->accessed)
			{
				AccelerationStructures.Add(InBinding);
			}
			break;

		default:
			// check(false);
			break;
		}
		break;

	case SPV_REFLECT_RESOURCE_FLAG_UAV:
		if (InBinding->uav_counter_binding)
		{
			AtomicCounters.Add(InBinding->uav_counter_binding);
		}

		switch (InBinding->descriptor_type)
		{
		case SPV_REFLECT_DESCRIPTOR_TYPE_STORAGE_IMAGE:
			TextureUAVs.Add(InBinding);
			break;

		case SPV_REFLECT_DESCRIPTOR_TYPE_STORAGE_TEXEL_BUFFER:
			TBufferUAVs.Add(InBinding);
			break;

		case SPV_REFLECT_DESCRIPTOR_TYPE_STORAGE_BUFFER:
			if (!AtomicCounters.Contains(InBinding) || InBinding->accessed)
			{
				SBufferUAVs.Add(InBinding);
			}
			break;

		default:
			break;
		}
		break;

	default:
		// Gather input attachments (e.g. subpass inputs)
		switch (InBinding->descriptor_type)
		{
		case SPV_REFLECT_DESCRIPTOR_TYPE_INPUT_ATTACHMENT:
			if (InBinding->accessed)
			{
				InputAttachments.Add(InBinding);
			}
			break;

		default:
			break;
		}
		break;
	} // switch
}

void FSpirvReflectBindings::GatherDescriptorBindings(const spv_reflect::ShaderModule& SpirvReflection)
{
	// Enumerate all binding points
	uint32 NumBindings = 0;
	SpvReflectResult SpvResult = SpirvReflection.EnumerateDescriptorBindings(&NumBindings, nullptr);
	check(SpvResult == SPV_REFLECT_RESULT_SUCCESS);

	if (NumBindings > 0)
	{
		TArray<SpvReflectDescriptorBinding*> Bindings;
		Bindings.SetNum(NumBindings);
		SpvResult = SpirvReflection.EnumerateDescriptorBindings(&NumBindings, Bindings.GetData());
		check(SpvResult == SPV_REFLECT_RESULT_SUCCESS);

		// Extract all the bindings first so that we process them in order - this lets us assign UAVs before other resources
		// Which is necessary to match the D3D binding scheme.
		for (SpvReflectDescriptorBinding* BindingEntry : Bindings)
		{
			AddDescriptorBinding(BindingEntry);
		}
	}
}

void FSpirvReflectBindings::GatherInputAttributes(const spv_reflect::ShaderModule& SpirvReflection)
{
	// Enumerate all input attributes
	uint32 NumInputAttributes = 0;
	SpvReflectResult SpvResult = SpirvReflection.EnumerateEntryPointInputVariables(SpirvReflection.GetEntryPointName(), &NumInputAttributes, nullptr);
	check(SpvResult == SPV_REFLECT_RESULT_SUCCESS);

	if (NumInputAttributes > 0)
	{
		InputAttributes.SetNum(NumInputAttributes);
		SpvResult = SpirvReflection.EnumerateEntryPointInputVariables(SpirvReflection.GetEntryPointName(), &NumInputAttributes, InputAttributes.GetData());
		check(SpvResult == SPV_REFLECT_RESULT_SUCCESS);
	}
}

void FSpirvReflectBindings::GatherOutputAttributes(const spv_reflect::ShaderModule & SpirvReflection)
{
	// Enumerate all output attributes
	uint32 NumOutputAttributes = 0;
	SpvReflectResult SpvResult = SpirvReflection.EnumerateEntryPointOutputVariables(SpirvReflection.GetEntryPointName(), &NumOutputAttributes, nullptr);
	check(SpvResult == SPV_REFLECT_RESULT_SUCCESS);

	if (NumOutputAttributes > 0)
	{
		OutputAttributes.SetNum(NumOutputAttributes);
		SpvResult = SpirvReflection.EnumerateEntryPointOutputVariables(SpirvReflection.GetEntryPointName(), &NumOutputAttributes, OutputAttributes.GetData());
		check(SpvResult == SPV_REFLECT_RESULT_SUCCESS);
	}
}

void FSpirvReflectBindings::GatherPushConstants(const spv_reflect::ShaderModule& SpirvReflection)
{
	// Enumerate all interface variables before filtering out push constants.
	uint32 NumVariables = 0;
	SpvReflectResult SpvResult = SpirvReflection.EnumerateEntryPointInterfaceVariables(SpirvReflection.GetEntryPointName(), &NumVariables, nullptr);
	check(SpvResult == SPV_REFLECT_RESULT_SUCCESS);

	if (NumVariables > 0)
	{
		TArray<SpvReflectInterfaceVariable*> AllVariables;
		AllVariables.SetNum(NumVariables);

		SpvResult = SpirvReflection.EnumerateEntryPointInterfaceVariables(SpirvReflection.GetEntryPointName(), &NumVariables, AllVariables.GetData());
		check(SpvResult == SPV_REFLECT_RESULT_SUCCESS);

		// Extract only the variables with the Push Constant storage class.
		for (SpvReflectInterfaceVariable* Variable : AllVariables)
		{
			if (Variable && Variable->storage_class == SpvStorageClassPushConstant)
			{
				PushConstants.Emplace(Variable);
			}
		}
	}
}

// Parse the index from a semantic name, e.g. "ATTRIBUTE14" returns 14.
static bool ParseSemanticIndex(const ANSICHAR* InName, int32& OutSemanticIndex)
{
	if (!InName)
	{
		return false;
	}

	for (int32 NameLen = FCStringAnsi::Strlen(InName), Index = NameLen; Index > 0; --Index)
	{
		if (!FChar::IsDigit(InName[Index - 1]))
		{
			if (Index == NameLen)
			{
				// Semantic name does not end with digits
				return false;
			}
			else
			{
				// Return suffix numeric starting at previous string position
				OutSemanticIndex = FCStringAnsi::Atoi(InName + Index);
				return true;
			}
		}
	}

	return false;
}

void FSpirvReflectBindings::AssignInputAttributeLocationsBySemanticIndex(spv_reflect::ShaderModule& SpirvReflection, const ANSICHAR* SemanticName)
{
	const int32 SemanticNameLen = FCStringAnsi::Strlen(SemanticName);
	for (SpvReflectInterfaceVariable* InterfaceVar : InputAttributes)
	{
		if (InterfaceVar->built_in == -1 && InterfaceVar->name && FCStringAnsi::Strncmp(InterfaceVar->name, SemanticName, SemanticNameLen) == 0)
		{
			int32 Location = 0;
			if (ParseSemanticIndex(InterfaceVar->name, Location))
			{
				SpirvReflection.ChangeInputVariableLocation(InterfaceVar, static_cast<uint32>(Location));
			}
		}
	}
}


