// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RHIDefinitions.h"


// Cross compiler support/common functionality
namespace CrossCompiler
{
extern SHADERCOMPILERCOMMON_API FCriticalSection* GetCrossCompilerLock();
}

