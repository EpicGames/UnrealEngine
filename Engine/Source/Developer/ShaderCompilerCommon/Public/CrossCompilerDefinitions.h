// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

//@todo-lh: As soon as HLSLcc is fully replaced by DXC for all shader compiler backends,
// move the content of this file into 'CrossCompilerDefinitions.h".
// Until then, we simply include that file from the HLSLcc folder since it only contains enum declaration but no further include dependencies.
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "ThirdParty/hlslcc/hlslcc/src/hlslcc_lib/HlslccDefinitions.h"
#endif
