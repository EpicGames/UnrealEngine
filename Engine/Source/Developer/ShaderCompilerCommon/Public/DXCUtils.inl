// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// not every shader format compiler dll will have access to DxcCreateInstance directly - some might be relying on GetProcAddress to find it
#ifndef DXCUTILS_HAS_DXCCREATEINSTANCE
	#define DXCUTILS_HAS_DXCCREATEINSTANCE 1
#endif

#include <String/BytesToHex.h>

#if DXCUTILS_HAS_DXCCREATEINSTANCE
static void DumpDebugBlobDetail(IDxcBlob* Blob, const TCHAR* BlobName)
#else
static void DumpDebugBlobDetail(IDxcBlob* Blob, const TCHAR* BlobName, TRefCountPtr<IDxcContainerReflection> Reflection)
#endif
{
	check(BlobName != nullptr);

	if (!FPlatformMisc::IsDebuggerPresent())
	{
		return;
	}

	if (Blob == nullptr)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Blob %s is NULL\n"), BlobName);
		return;
	}

#if DXCUTILS_HAS_DXCCREATEINSTANCE
	TRefCountPtr<IDxcContainerReflection> Reflection;
	DxcCreateInstance(CLSID_DxcContainerReflection, __uuidof(IDxcContainerReflection), (void**)Reflection.GetInitReference());
#endif
	if (!Reflection.IsValid())
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Can't show debug detail for Blob %s - Failed to create IDxcContainerReflection\n"), BlobName);
		return;
	}

	Reflection->Load(Blob);

	UINT32 PartCount = 0;
	Reflection->GetPartCount(&PartCount);

	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Blob %s : %d parts, %d total bytes\n"), BlobName, PartCount, Blob->GetBufferSize());
	for (UINT32 PartIndex = 0; PartIndex < PartCount; PartIndex++)
	{
		UINT32 PartKind;
		Reflection->GetPartKind(PartIndex, &PartKind);

		const TCHAR* BlobDescription;
		switch (PartKind)
		{
		case DXC_FOURCC('D', 'X', 'B', 'C'): BlobDescription = TEXT("Container (back-compat)"); break;
		case DXC_FOURCC('R', 'D', 'E', 'F'): BlobDescription = TEXT("ResourceDef");             break;
		case DXC_FOURCC('I', 'S', 'G', '1'): BlobDescription = TEXT("InputSignature");          break;
		case DXC_FOURCC('O', 'S', 'G', '1'): BlobDescription = TEXT("OutputSignature");         break;
		case DXC_FOURCC('P', 'S', 'G', '1'): BlobDescription = TEXT("PatchConstantSignature");  break;
		case DXC_FOURCC('S', 'T', 'A', 'T'): BlobDescription = TEXT("ShaderStatistics");        break;
		case DXC_FOURCC('I', 'L', 'D', 'B'): BlobDescription = TEXT("ShaderDebugInfoDXIL");     break;
		case DXC_FOURCC('I', 'L', 'D', 'N'): BlobDescription = TEXT("ShaderDebugName");         break;
		case DXC_FOURCC('S', 'F', 'I', '0'): BlobDescription = TEXT("FeatureInfo");             break;
		case DXC_FOURCC('U', 'S', 'E', 'R'): BlobDescription = TEXT("UserInfo");                break;
		case DXC_FOURCC('P', 'R', 'I', 'V'): BlobDescription = TEXT("PrivateData");             break;
		case DXC_FOURCC('R', 'T', 'S', '0'): BlobDescription = TEXT("RootSignature");           break;
		case DXC_FOURCC('D', 'X', 'I', 'L'): BlobDescription = TEXT("DXIL");                    break;
		case DXC_FOURCC('P', 'S', 'V', '0'): BlobDescription = TEXT("PipelineStateValidation"); break;
		case DXC_FOURCC('R', 'D', 'A', 'T'): BlobDescription = TEXT("RuntimeData");             break;
		case DXC_FOURCC('H', 'A', 'S', 'H'): BlobDescription = TEXT("ShaderHash");              break;
		default:                             BlobDescription = TEXT("(unknown)");               break;
		}

		TRefCountPtr<IDxcBlob> PartBlob;
		Reflection->GetPartContent(PartIndex, PartBlob.GetInitReference());

		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("\t[%4d] %c%c%c%c (%s) %d bytes\n"), PartIndex, (PartKind & 0xFF), (PartKind >> 8) & 0xFF, (PartKind >> 16) & 0xFF, (PartKind >> 24) & 0xFF, BlobDescription, PartBlob->GetBufferSize());
	}
	FPlatformMisc::LowLevelOutputDebugString(TEXT("\n\n"));
}

static HRESULT RetrieveDebugNameAndBlob(TRefCountPtr<IDxcResult>& CompileResult, FString& OutDebugBlobName, IDxcBlob** OutDebugBlob, DxcShaderHash& ShaderHash)
{
	check(CompileResult->HasOutput(DXC_OUT_SHADER_HASH));
	{
		TRefCountPtr<IDxcBlob> HashBlob;
		TRefCountPtr<IDxcBlobUtf16> OutputNameUnused;

		HRESULT HashResult = CompileResult->GetOutput(DXC_OUT_SHADER_HASH, __uuidof(IDxcBlob), (void**)HashBlob.GetInitReference(), OutputNameUnused.GetInitReference());
		check(SUCCEEDED(HashResult));
		check(HashBlob->GetBufferSize() == sizeof(DxcShaderHash));
		ShaderHash = *reinterpret_cast<DxcShaderHash*>(HashBlob->GetBufferPointer());
	}

	HRESULT Result = S_OK;
	if (OutDebugBlob && CompileResult->HasOutput(DXC_OUT_PDB))
	{
		// If the PDB blob exists, we get its name at the same time.
		TRefCountPtr<IDxcBlobUtf16> DebugBlobName;
		Result = CompileResult->GetOutput(DXC_OUT_PDB, __uuidof(IDxcBlob), (void**)OutDebugBlob, DebugBlobName.GetInitReference());
		if (SUCCEEDED(Result))
		{
			OutDebugBlobName = DebugBlobName->GetStringPointer();
		}
	}
	else
	{
		// If the PDB blob is not available, or if we don't need to return it, we can reconstruct the name 
		// the PDB would have used from the Shader Hash
		TStringBuilder<128> PdbNameBuilder;
		UE::String::BytesToHex(TConstArrayView<uint8>(ShaderHash.HashDigest, sizeof(ShaderHash.HashDigest)), PdbNameBuilder);
		PdbNameBuilder << TEXT(".pdb");
		OutDebugBlobName = PdbNameBuilder.ToString();
	}
	return Result;
}
