// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/RefCounting.h"

#define UE_API SHADERCOMPILERCOMMON_API

struct FDllHandle : public FRefCountedObject
{
private:
	void* Handle = nullptr;

public:
	FDllHandle(const TCHAR* InFilename);
	virtual ~FDllHandle();
};

class FDxcModuleWrapper
{
private:
	uint32 ModuleVersionHash = 0;
protected:
	FORCEINLINE uint32 GetModuleVersionHash() const
	{
		return ModuleVersionHash;
	}
public:
	UE_API FDxcModuleWrapper();
	UE_API virtual ~FDxcModuleWrapper();
};

class FShaderConductorModuleWrapper : private FDxcModuleWrapper
{
private:
	uint32 ModuleVersionHash = 0;
protected:
	FORCEINLINE uint32 GetModuleVersionHash() const
	{
		return ModuleVersionHash;
	}
public:
	UE_API FShaderConductorModuleWrapper();
	UE_API virtual ~FShaderConductorModuleWrapper();
};

#undef UE_API
