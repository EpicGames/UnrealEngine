// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CrossCompilerCommon.h"
#include "HAL/Platform.h"
#include "Serialization/Archive.h"

#define UE_API SHADERCOMPILERCOMMON_API

// Forward declaration from <spirv_reflect.h>
struct SpvReflectBlockVariable;
struct SpvReflectInterfaceVariable;
struct SpvReflectTypeDescription;

// Cross compiler support/common functionality
namespace CrossCompiler
{
	class FHlslccHeaderWriter
	{
	public:

		UE_API void WriteSourceInfo(const TCHAR* VirtualSourceFilePath, const TCHAR* EntryPointName);
		UE_API void WriteCompilerInfo(const TCHAR* CompilerName = TEXT("ShaderConductor"));

		UE_API void WriteInputAttribute(const SpvReflectInterfaceVariable& Attribute);
		UE_API void WriteInputAttribute(const TCHAR* AttributeName, const TCHAR* TypeSpecifier, int32 Location, bool bLocationPrefix, bool bLocationSuffix);
		UE_API void WriteOutputAttribute(const SpvReflectInterfaceVariable& Attribute);
		UE_API void WriteOutputAttribute(const TCHAR* AttributeName, const TCHAR* TypeSpecifier, int32 Location, bool bLocationPrefix, bool bLocationSuffix);
		UE_API void WriteUniformBlock(const TCHAR* ResourceName, uint32 BindingIndex);
		UE_API void WritePackedGlobal(const TCHAR* ResourceName, EPackedTypeName PackedType, uint32 ByteOffset, uint32 ByteSize);
		UE_API void WritePackedGlobal(const SpvReflectBlockVariable& Variable);
		UE_API void WritePackedUB(uint32 BindingIndex);
		UE_API void WritePackedUBField(const TCHAR* ResourceName, uint32 ByteOffset, uint32 ByteSize);
		UE_API void WritePackedUB(const FString& UBName, uint32 BindingIndex);
		UE_API void WritePackedUBField(const FString& UBName, const TCHAR* ResourceName, uint32 ByteOffset, uint32 ByteSize);
		UE_API void WritePackedUBCopy(uint32 SourceCB, uint32 SourceOffset, uint32 DestCBIndex, uint32 DestCBPrecision, uint32 DestOffset, uint32 Size, bool bGroupFlattenedUBs = false);
		UE_API void WritePackedUBGlobalCopy(uint32 SourceCB, uint32 SourceOffset, uint32 DestCBIndex, uint32 DestCBPrecision, uint32 DestOffset, uint32 Size, bool bGroupFlattenedUBs = false);
		UE_API void WriteSRV(const TCHAR* ResourceName, uint32 BindingIndex, uint32 Count = 1);
		UE_API void WriteSRV(const TCHAR* ResourceName, uint32 BindingIndex, uint32 Count, const TArray<FString>& AssociatedResourceNames);
		UE_API void WriteUAV(const TCHAR* ResourceName, uint32 BindingIndex, uint32 Count = 1);
		UE_API void WriteSamplerState(const TCHAR* ResourceName, uint32 BindingIndex);
		UE_API void WriteNumThreads(uint32 NumThreadsX, uint32 NumThreadsY, uint32 NumThreadsZ);
		UE_API void WriteAccelerationStructures(const TCHAR* ResourceName, uint32 BindingIndex);

		UE_API void WriteSideTable(const TCHAR* ResourceName, uint32 SideTableIndex);
		UE_API void WriteArgumentBuffers(uint32 BindingIndex, const TArray<uint32>& ResourceIndices);

		/** Returns the finalized meta data. */
		UE_API FString ToString() const;
		static UE_API EPackedTypeName EncodePackedGlobalType(const SpvReflectTypeDescription& TypeDescription, bool bHalfPrecision = false);

		static UE_API void WriteIOAttribute(FString& OutMetaData, const SpvReflectInterfaceVariable& Attribute, bool bIsInput);

	private:
		void WriteIOAttribute(FString& OutMetaData, const TCHAR* AttributeName, const TCHAR* TypeSpecifier, int32 Location, bool bLocationPrefix, bool bLocationSuffix);

	private:
		struct FMetaDataStrings
		{
			FString SourceInfo;
			FString CompilerInfo;
			FString InputAttributes;
			FString OutputAttributes;
			FString UniformBlocks;
			FString PackedGlobals;
			TMap<FString, FString> PackedUBs;
			TMap<FString, FString> PackedUBFields;
			FString PackedUBCopies;
			FString PackedUBGlobalCopies;
			FString SRVs; // Shader resource views (SRV) and samplers
			FString UAVs; // Unordered access views (UAV)
			FString SamplerStates;
			FString NumThreads;
			FString ExternalTextures; // External texture resources (Vulkan ES3.1 profile only)
			FString SideTable; // Side table for additional indices, e.,g. "spvBufferSizeConstants(31)" (Metal only)
			FString ArgumentBuffers; // Indirect argument buffers (Metal only)
			FString AccelerationStructures;
		};

	private:
		FMetaDataStrings Strings;
		
	};
}

#undef UE_API
