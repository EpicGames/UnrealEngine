// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum class EInstructionSet : uint8
{
	RDNA1,
	RDNA2,
};

namespace ISAParser
{
	extern SHADERCOMPILERCOMMON_API bool HasDerivativeOps(bool& bHasDerivativeOps, const char* Code, uint32 CodeLength, EInstructionSet InstructionSet);
}
