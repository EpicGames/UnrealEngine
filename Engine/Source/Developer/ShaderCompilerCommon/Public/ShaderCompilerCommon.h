// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ShaderCore.h"
#include "ShaderCompilerCore.h"
#include "ShaderConductorContext.h"
#include "Templates/Function.h"
#include "Interfaces/IShaderFormat.h"

#define UE_API SHADERCOMPILERCOMMON_API

class FShaderParameterParser;
class FShaderSource;

namespace UE::ShaderCompilerCommon
{
	static constexpr const TCHAR* kUniformBufferConstantBufferPrefix = TEXT("UniformBufferConstants_");
	static constexpr const TCHAR* kPlatformHashStatName = TEXT("PlatformHash");
}

/**
 * This function looks for resources specified in ResourceTableMap in the 
 * parameter map, adds them to the resource table, and removes them from the
 * parameter map. If a resource is used from a currently unmapped uniform 
 * buffer we allocate a slot for it from UsedUniformBufferSlots.
 * Returns false if there's any internal error.
 */
extern SHADERCOMPILERCOMMON_API bool BuildResourceTableMapping(
		const FShaderResourceTableMap& ResourceTableMap,
		const TMap<FString,FUniformBufferEntry>& UniformBufferMap,
		TBitArray<>& UsedUniformBufferSlots,
		FShaderParameterMap& ParameterMap,
		FShaderCompilerResourceTable& OutSRT
	);

/** Culls global uniform buffer entries from the parameter map. */
extern SHADERCOMPILERCOMMON_API void CullGlobalUniformBuffers(const TMap<FString, FUniformBufferEntry>& UniformBufferMap, FShaderParameterMap& ParameterMap);

/**
 * Builds a token stream out of the resource map. The resource map is one
 * of the arrays generated by BuildResourceTableMapping. The token stream
 * is used at runtime to gather resources from tables and bind them to the
 * appropriate slots.
 */
extern SHADERCOMPILERCOMMON_API void BuildResourceTableTokenStream(
	const TArray<uint32>& InResourceMap,
	int32 MaxBoundResourceTable,
	TArray<uint32>& OutTokenStream,
	bool bGenerateEmptyTokenStreamIfNoResources = false
	);

// Finds the number of used uniform buffers in a resource map
extern SHADERCOMPILERCOMMON_API int16 GetNumUniformBuffersUsed(const FShaderCompilerResourceTable& InSRT);

namespace UE::ShaderCompilerCommon
{
	extern SHADERCOMPILERCOMMON_API void BuildShaderResourceTable(const FShaderCompilerResourceTable& GenericSRT, FShaderResourceTable& OutSRT, bool bGenerateEmptyTokenStreamIfNoResources = false);

	extern SHADERCOMPILERCOMMON_API bool ExecuteShaderPreprocessingSteps(
		FShaderPreprocessOutput& PreprocessOutput,
		const FShaderCompilerInput& Input,
		const FShaderCompilerEnvironment& Environment,
		const FShaderCompilerDefinitions& AdditionalDefines
		);

	extern SHADERCOMPILERCOMMON_API bool ExecuteShaderPreprocessingSteps(
		FShaderPreprocessOutput& PreprocessOutput,
		const FShaderCompilerInput& Input,
		const FShaderCompilerEnvironment& Environment
		);

	extern SHADERCOMPILERCOMMON_API FStringView          RemoveConstantBufferPrefix(FStringView InName);
	extern SHADERCOMPILERCOMMON_API FString              RemoveConstantBufferPrefix(const FString& InName);

	extern SHADERCOMPILERCOMMON_API bool                 ValidatePackedResourceCounts(FShaderCompilerOutput& Output, const FShaderCodePackedResourceCounts& PackedResourceCounts);

	extern SHADERCOMPILERCOMMON_API void ParseRayTracingEntryPoint(const FString& Input, FString& OutMain, FString& OutAnyHit, FString& OutIntersection);

	/*
	* Parses ray tracing shader entry point specification string in one of the following formats:
	* 1) Verbatim single entry point name, e.g. "MainRGS"
	* 2) Complex entry point for ray tracing hit group shaders:
	*      a) "closesthit=MainCHS"
	*      b) "closesthit=MainCHS anyhit=MainAHS"
	*      c) "closesthit=MainCHS anyhit=MainAHS intersection=MainIS"
	*      d) "closesthit=MainCHS intersection=MainIS"
	*    NOTE: closesthit attribute must always be provided for complex hit group entry points
	*/
	extern SHADERCOMPILERCOMMON_API void ParseRayTracingEntryPoint(const FStringView& Input, FStringView& OutMain, FStringView& OutAnyHit, FStringView& OutIntersection);

	/**
	* Rewrites a fully preprocessed shader source code, removing any functions or structs that are not reachable from a given entry point or list of symbols.
	* This is a high-level wrapper for UE::ShaderMinifier that should be used in all shader compiler back-ends for consistency.
	* Any errors encountered during parsing or minification are added to OutErrors.
	* InOutPreprocessedShaderSource is replaced with the rewritten code on success and is kept intact on failure.
	*/
	extern SHADERCOMPILERCOMMON_API bool RemoveDeadCode(FShaderSource& InOutPreprocessedShaderSource, const FString& EntryPoint, TArray<FShaderCompilerError>& OutErrors);
	extern SHADERCOMPILERCOMMON_API bool RemoveDeadCode(FShaderSource& InOutPreprocessedShaderSource, const FString& EntryPoint, TConstArrayView<FStringView> RequiredSymbols, TArray<FShaderCompilerError>& OutErrors);
	extern SHADERCOMPILERCOMMON_API bool RemoveDeadCode(FShaderSource& InOutPreprocessedShaderSource, TConstArrayView<FStringView> RequiredSymbols, TArray<FShaderCompilerError>& OutErrors);
	
	struct FDebugShaderDataOptions
	{
		struct FAdditionalOutput
		{
			const TCHAR* BaseFileName;
			const TCHAR* Data;
		};

		const TCHAR* OverrideBaseFilename = nullptr;
		const TCHAR* FilenamePrefix = nullptr;
		TFunction<FString()> AppendPreSource{};
		TFunction<FString()> AppendPostSource{};
		TArray<FAdditionalOutput> AdditionalOutputs;
		UE_DEPRECATED(5.7, "bSourceOnly flag on debug data options is no longer supported")
		bool bSourceOnly = false;

		SHADERCOMPILERCOMMON_API FString GetDebugShaderPath(const FShaderCompilerInput& Input, const TCHAR* Suffix = nullptr) const;
	};

	UE_DEPRECATED(5.7, "DumpExtendedDebugShaderData now requires a FShaderDebugDataContext struct parameter")
		extern SHADERCOMPILERCOMMON_API void DumpExtendedDebugShaderData(
			const FShaderCompilerInput& Input,
			const FShaderPreprocessOutput& PreprocessOutput,
			const FShaderCompilerOutput& Output,
			const UE::ShaderCompilerCommon::FDebugShaderDataOptions& Options = FDebugShaderDataOptions());

	/*
	 * Dumps extended debug information; including all outputs from DumpDebugShaderData as well as the following:
	 *		- OutputHash.txt file containing the SHA hash of the shader job output
	 *		- Diagnostics.txt file containing all errors/warnings encountered for the job
	 *		  (if EShaderDebugInfoFlags::Diagnostics is set on Input.DebugInfoFlags)
	 *		- InputHash.txt file containing the hash used as the key to the shader job cache
	 *		  (if EShaderDebugInfoFlags::InputHash is set on Input.DebugInfoFlags and specified InputHash is non-empty)
	 *		- any outputs specified on the AdditionalOutputs array in the Options struct
	 * This is intended to be used by shader formats implementing the independent preprocessing API.
	 */
	extern SHADERCOMPILERCOMMON_API void DumpExtendedDebugShaderData(
		const FShaderCompilerInput& Input,
		const FShaderPreprocessOutput& PreprocessOutput,
		const FShaderCompilerOutput& Output,
		FShaderDebugDataContext& Ctx,
		const UE::ShaderCompilerCommon::FDebugShaderDataOptions& Options = FDebugShaderDataOptions());

	UE_DEPRECATED(5.7, "No longer used")
	extern SHADERCOMPILERCOMMON_API void SerializeEnvironmentFromBase64(FShaderCompilerEnvironment& Env, const FString& DebugUSF);
	UE_DEPRECATED(5.7, "No longer used")
	extern SHADERCOMPILERCOMMON_API FString SerializeEnvironmentToBase64(const FShaderCompilerEnvironment& Env);

	/*
	 * Constructs the modified preprocessed source that would be dumped to a .usf file via DumpDebugShaderData, including the following additions:
	 * - comments containing the names and values of all defines when preprocessing this shader
	 * - a comment containing the FShaderCompilerInput DebugDescription
	 * - any additions performed by the AppendPreSource/AppendPostSource TFunctions
	 * @param	Input The input of the compilation job
	 * @param	PreprocessedSource The unmodified preprocessed source (used as input to the compilation)
	 * @param	Options Options which can change behaviour of the debug dump; see above.
	 */
	extern SHADERCOMPILERCOMMON_API FString GetDebugShaderContents(const FShaderCompilerInput& Input, FStringView PreprocessedSource, const FDebugShaderDataOptions& Options = FDebugShaderDataOptions(), const TCHAR* Suffix = nullptr);
	
	/*
	 * Runs a few validation steps and reports any known issues such as parameter names that might cause problems on certain backend compilers, especially the old FXC compiler.
	 */
	extern SHADERCOMPILERCOMMON_API bool ValidateShaderAgainstKnownIssues(const FString& InSourceCode, TArray<FString>& OutErrors, const TCHAR* InSourceCodeFilename = nullptr);

	class FBaseShaderFormat : public IShaderFormat
	{
	public:
		virtual SHADERCOMPILERCOMMON_API bool PreprocessShader(
			const FShaderCompilerInput& Input,
			const FShaderCompilerEnvironment& Environment,
			FShaderPreprocessOutput& PreprocessOutput) const override;

		virtual SHADERCOMPILERCOMMON_API void OutputDebugData(
			const FShaderCompilerInput& Input,
			const FShaderPreprocessOutput& PreprocessOutput,
			const FShaderCompilerOutput& Output,
			FShaderDebugDataContext& Ctx) const override;

		virtual SHADERCOMPILERCOMMON_API void OutputDebugDataMinimal(const FShaderCompilerInput& Input, FShaderDebugDataContext& Ctx) const override;
	};
}

enum class EUniformBufferMemberReflectionReason
{
	None            	= 0,
	NeedsReflection 	= 1 << 0,
	Bindless        	= 1 << 1,
	MaterialUBBindError = 1 << 2,
};
ENUM_CLASS_FLAGS(EUniformBufferMemberReflectionReason);

SHADERCOMPILERCOMMON_API EUniformBufferMemberReflectionReason ShouldReflectUniformBufferMembers(
	const FShaderCompilerInput& Input,
	FStringView UniformBufferName
);

extern SHADERCOMPILERCOMMON_API void HandleReflectedGlobalConstantBufferMember(
	const FShaderCompilerInput& Input,
	const FShaderParameterParser& ShaderParameterParser,
	FStringView MemberName,
	uint32 ConstantBufferIndex,
	int32 ReflectionOffset,
	int32 ReflectionSize,
	FShaderCompilerOutput& CompilerOutput
);

UE_DEPRECATED(5.8, "HandleReflectedGlobalConstantBufferMember now needs Input and ShaderParameterParser parameters for future features")
extern SHADERCOMPILERCOMMON_API void HandleReflectedGlobalConstantBufferMember(
	FStringView MemberName,
	uint32 ConstantBufferIndex,
	int32 ReflectionOffset,
	int32 ReflectionSize,
	FShaderCompilerOutput& CompilerOutput
);

extern SHADERCOMPILERCOMMON_API void HandleReflectedUniformBufferConstantBufferMember(
	const FShaderCompilerInput& Input,
	EUniformBufferMemberReflectionReason Reason,
	FStringView UniformBufferName,
	int32 UniformBufferSlot,
	FStringView MemberName,
	int32 ReflectionOffset,
	int32 ReflectionSize,
	FShaderCompilerOutput& CompilerOutput
);

extern SHADERCOMPILERCOMMON_API void HandleReflectedRootConstantBufferMember(
	const FShaderCompilerInput& Input,
	const FShaderParameterParser& ShaderParameterParser,
	FStringView MemberName,
	int32 ReflectionOffset,
	int32 ReflectionSize,
	FShaderCompilerOutput& CompilerOutput
);

extern SHADERCOMPILERCOMMON_API void HandleReflectedRootConstantBuffer(
	int32 ConstantBufferSize,
	FShaderCompilerOutput& CompilerOutput
);

extern SHADERCOMPILERCOMMON_API void HandleReflectedUniformBuffer(
	const FShaderCompilerInput& Input,
	FStringView UniformBufferName,
	int32 ReflectionSlot,
	int32 BaseIndex,
	int32 BufferSize,
	FShaderCompilerOutput& CompilerOutput
);

inline void HandleReflectedUniformBuffer(const FShaderCompilerInput& Input, FStringView UniformBufferName, int32 ReflectionSlot, int32 BufferSize, FShaderCompilerOutput& CompilerOutput)
{
	HandleReflectedUniformBuffer(Input, UniformBufferName, ReflectionSlot, 0, BufferSize, CompilerOutput);
}

inline void HandleReflectedUniformBuffer(const FShaderCompilerInput& Input, FStringView UniformBufferName, int32 ReflectionSlot, FShaderCompilerOutput& CompilerOutput)
{
	HandleReflectedUniformBuffer(Input, UniformBufferName, ReflectionSlot, 0, CompilerOutput);
}

extern SHADERCOMPILERCOMMON_API void HandleReflectedShaderResource(
	FStringView ResourceName,
	int32 BindOffset,
	int32 ReflectionSlot,
	int32 BindCount,
	FShaderCompilerOutput& CompilerOutput
);

extern SHADERCOMPILERCOMMON_API void UpdateStructuredBufferStride(
	const FShaderCompilerInput& Input,
	FStringView ResourceName,
	uint16 BindPoint,
	uint16 Stride,
	FShaderCompilerOutput& CompilerOutput
);

extern SHADERCOMPILERCOMMON_API void AddShaderValidationSRVType(
	uint16 BindPoint,
	EShaderCodeResourceBindingType TypeDecl,
	FShaderCompilerOutput& CompilerOutput);

extern SHADERCOMPILERCOMMON_API void AddShaderValidationUAVType(
	uint16 BindPoint,
	EShaderCodeResourceBindingType TypeDecl,
	FShaderCompilerOutput& CompilerOutput);

extern SHADERCOMPILERCOMMON_API void AddShaderValidationUBSize(
	uint16 BindPoint,
	uint32_t Size,
	FShaderCompilerOutput& CompilerOutput);

inline void HandleReflectedShaderResource(FStringView ResourceName, int32 ReflectionSlot, int32 BindCount, FShaderCompilerOutput& CompilerOutput)
{
	HandleReflectedShaderResource(ResourceName, 0, ReflectionSlot, BindCount, CompilerOutput);
}

inline void HandleReflectedShaderResource(FStringView ResourceName, int32 ReflectionSlot, FShaderCompilerOutput& CompilerOutput)
{
	HandleReflectedShaderResource(ResourceName, ReflectionSlot, 1, CompilerOutput);
}

extern SHADERCOMPILERCOMMON_API void HandleReflectedShaderUAV(
	FStringView UAVName,
	int32 BindOffset,
	int32 ReflectionSlot,
	int32 BindCount,
	FShaderCompilerOutput& CompilerOutput
);

inline void HandleReflectedShaderUAV(FStringView UAVName, int32 ReflectionSlot, int32 BindCount, FShaderCompilerOutput& CompilerOutput)
{
	HandleReflectedShaderUAV(UAVName, 0, ReflectionSlot, BindCount, CompilerOutput);
}

inline void HandleReflectedShaderUAV(FStringView UAVName, int32 ReflectionSlot, FShaderCompilerOutput& CompilerOutput)
{
	HandleReflectedShaderUAV(UAVName, ReflectionSlot, 1, CompilerOutput);
}

extern SHADERCOMPILERCOMMON_API void HandleReflectedShaderSampler(
	FStringView SamplerName,
	int32 BindOffset,
	int32 ReflectionSlot,
	int32 BindCount,
	FShaderCompilerOutput& CompilerOutput
);

inline void HandleReflectedShaderSampler(FStringView SamplerName, int32 ReflectionSlot, int32 BindCount, FShaderCompilerOutput& CompilerOutput)
{
	HandleReflectedShaderSampler(SamplerName, 0, ReflectionSlot, BindCount, CompilerOutput);
}

inline void HandleReflectedShaderSampler(FStringView SamplerName, int32 ReflectionSlot, FShaderCompilerOutput& CompilerOutput)
{
	HandleReflectedShaderSampler(SamplerName, ReflectionSlot, 1, CompilerOutput);
}

/** Adds a note to CompilerOutput.Error about where the shader parameter structure is on C++ side. */
extern SHADERCOMPILERCOMMON_API void AddNoteToDisplayShaderParameterStructureOnCppSide(
	const FShaderParametersMetadata* ParametersStructure,
	FShaderCompilerOutput& CompilerOutput);

/** Adds an error to CompilerOutput.Error about a shader parameters that could not be bound. */
extern SHADERCOMPILERCOMMON_API void AddUnboundShaderParameterError(
	const FShaderCompilerInput& CompilerInput,
	const FShaderParameterParser& ShaderParameterParser,
	FStringView ParameterBindingName,
	FShaderCompilerOutput& CompilerOutput);

// Convert generated UniformBuffer code and references into something the shader compilers can use.
extern SHADERCOMPILERCOMMON_API void CleanupUniformBufferCode(const FShaderCompilerEnvironment& Environment, FShaderSource& PreprocessedShaderSource);

template <typename CharType>
const CharType* FindMatchingBlock(const CharType* OpeningCharPtr, char OpenChar, char CloseChar)
{
	const CharType* SearchPtr = OpeningCharPtr;
	int32 Depth = 0;

	while (*SearchPtr)
	{
		if (*SearchPtr == OpenChar)
		{
			Depth++;
		}
		else if (*SearchPtr == CloseChar)
		{
			if (Depth == 0)
			{
				return SearchPtr;
			}

			Depth--;
		}
		SearchPtr++;
	}

	return nullptr;
}

template <typename CharType>
const CharType* FindMatchingClosingBrace(const CharType* OpeningCharPtr) { return FindMatchingBlock<CharType>(OpeningCharPtr, '{', '}'); };

extern SHADERCOMPILERCOMMON_API const TCHAR* ParseHLSLSymbolName(const TCHAR* SearchString, FString& SymboName);
extern SHADERCOMPILERCOMMON_API void ParseHLSLTypeName(const TCHAR* SearchString, const TCHAR*& TypeNameStartPtr, const TCHAR*& TypeNameEndPtr);
extern SHADERCOMPILERCOMMON_API FStringView FindNextHLSLDefinitionOfType(FStringView Typename, FStringView StartPos);

// Structure to hold forward declarations for a specific scope/namespace chain for the HlslParser
struct UE_DEPRECATED(5.7, "FScopedDeclarations has been moved into internal module HlslParser and is no longer supported as public interface.") FScopedDeclarations
{
	FScopedDeclarations(TConstArrayView<FStringView> InScope, TConstArrayView<FStringView> InSymbols)
		: Scope(InScope)
		, Symbols(InSymbols)
	{
	}
	TConstArrayView<FStringView> Scope;
	TConstArrayView<FStringView> Symbols;
};

UE_DEPRECATED(5.7, "RemoveUnusedOutputs() has been moved into internal module HlslParser and is no longer supported as public interface.")
inline bool RemoveUnusedOutputs(...)
{
	return false; // dummy
}

UE_DEPRECATED(5.7, "RemoveUnusedInputs() has been moved into internal module HlslParser and is no longer supported as public interface.")
inline bool RemoveUnusedInputs(...)
{
	return false; // dummy
}

// Shader input/output parameter storage classes. Naming adopted from SPIR-V nomenclature.
enum class UE_DEPRECATED(5.7, "EShaderParameterStorageClass has been moved into internal module HlslParser and is no longer supported as public interface.") EShaderParameterStorageClass
{
	Input,
	Output,
};

// Returns the semantic names of all individual entry point parameters (i.e. all structure fields are inlined)
UE_DEPRECATED(5.7, "FindEntryPointParameters() has been moved into internal module HlslParser and is no longer supported as public interface.")
inline bool FindEntryPointParameters(...)
{
	return false; //dummy
}

extern SHADERCOMPILERCOMMON_API bool ConvertFromFP32ToFP16(FString& InOutSourceCode, TArray<FString>& OutErrors);

// EXPERIMENTAL: Adds a new function to the shader where calls to `FunctionToInline` from `EntryPoint` are inlined using AST manipulation
// Current limitations:
// - Inlining from nested compound statement scopes is not supported (currently only trivial call statements in the body are processed)
// - Functions with return statements cannot be inlined
// - Multiple levels of inlining are not supported
extern SHADERCOMPILERCOMMON_API bool InlineFunction(FString& InOutSourceCode, FString& InOutEntryPoint, const FStringView FunctionToInline, TArray<FString>& OutErrors);

enum class EShaderConductorTarget
{
	Dxil,
	Spirv,
};
extern SHADERCOMPILERCOMMON_API void WriteShaderConductorCommandLine(const FShaderCompilerInput& Input, const FString& SourceFilename, EShaderConductorTarget Target);

extern SHADERCOMPILERCOMMON_API void DumpDebugShaderText(const FShaderCompilerInput& Input, const FString& InSource, const FString& FileExtension);
extern SHADERCOMPILERCOMMON_API void DumpDebugShaderText(const FShaderCompilerInput& Input, ANSICHAR* InSource, int32 InSourceLength, const FString& FileExtension); 
extern SHADERCOMPILERCOMMON_API void DumpDebugShaderText(const FShaderCompilerInput& Input, ANSICHAR* InSource, int32 InSourceLength, const FString& FileName, const FString& FileExtension);
extern SHADERCOMPILERCOMMON_API void DumpDebugShaderBinary(const FShaderCompilerInput& Input, void* InData, int32 InDataByteSize, const FString& FileExtension);
extern SHADERCOMPILERCOMMON_API void DumpDebugShaderBinary(const FShaderCompilerInput& Input, void* InData, int32 InDataByteSize, const FString& FileName, const FString& FileExtension);
extern SHADERCOMPILERCOMMON_API void DumpDebugShaderDisassembledSpirv(const FShaderCompilerInput& Input, void* InData, int32 InDataByteSize, const FString& FileExtension);
extern SHADERCOMPILERCOMMON_API void DumpDebugShaderDisassembledDxil(const FShaderCompilerInput& Input, void* InData, int32 InDataByteSize, const FString& FileExtension);

// calls 'Offline Compiler' to compile the source code and extract the stats
extern SHADERCOMPILERCOMMON_API void CompileShaderOffline(const FShaderCompilerInput& Input,
	FShaderCompilerOutput& ShaderOutput,
	const ANSICHAR* ShaderSource,
	const int32 SourceSize,
	bool bVulkanSpirV,
	const ANSICHAR* VulkanSpirVEntryPoint = nullptr);

// Cross compiler support/common functionality
namespace CrossCompiler
{
	extern SHADERCOMPILERCOMMON_API void ParseHlslccError(TArray<FShaderCompilerError>& OutErrors, const FString& InLine, bool bUseAbsolutePaths = false);

	struct FHlslccHeader
	{
		UE_API FHlslccHeader();
		virtual ~FHlslccHeader() = default;

		UE_API bool Read(const ANSICHAR*& ShaderSource, int32 SourceLen);

		// After the standard header, different backends can output their own info
		virtual bool ParseCustomHeaderEntries(const ANSICHAR*& ShaderSource)
		{
			return true;
		}

		struct FInOut
		{
			FString Type;
			int32 Index;
			int32 ArrayCount;
			FString Name;
		};

		struct FAttribute
		{
			int32 Index;
			FString Name;
		};

		struct FPackedGlobal
		{
			ANSICHAR PackedType;
			FString Name;
			int32 Offset;
			int32 Count;
		};

		//struct FUniform
		//{
		//};

		struct FPackedUB
		{
			FAttribute Attribute;
			struct FMember
			{
				FString Name;
				int32 Offset;
				int32 Count;
			};
			TArray<FMember> Members;
		};

		struct FPackedUBCopy
		{
			int32 SourceUB;
			int32 SourceOffset;
			int32 DestUB;
			ANSICHAR DestPackedType;
			int32 DestOffset;
			int32 Count;
		};

		struct FSampler
		{
			FString Name;
			int32 Offset;
			int32 Count;
			TArray<FString> SamplerStates;
		};

		struct FUAV
		{
			FString Name;
			int32 Offset;
			int32 Count;
		};

		struct FAccelerationStructure
		{
			FString Name;
			int32 Offset = 0;
		};

		FString Name;
		TArray<FInOut> Inputs;
		TArray<FInOut> Outputs;
		TArray<FAttribute> UniformBlocks;
		//TArray<FUniform> Uniforms;
		TArray<FPackedGlobal> PackedGlobals;
		TArray<FPackedUB> PackedUBs;
		TArray<FPackedUBCopy> PackedUBCopies;
		TArray<FPackedUBCopy> PackedUBGlobalCopies;
		TArray<FSampler> Samplers;
		TArray<FUAV> UAVs;
		TArray<FAttribute> SamplerStates;
		TArray<FAccelerationStructure> AccelerationStructures;
		uint32 NumThreads[3];

		static UE_API bool ReadInOut(const ANSICHAR*& ShaderSource, TArray<FInOut>& OutAttributes);
		static UE_API bool ReadCopies(const ANSICHAR*& ShaderSource, bool bGlobals, TArray<FPackedUBCopy>& OutCopies);
	};

	extern SHADERCOMPILERCOMMON_API const TCHAR* GetFrequencyName(EShaderFrequency Frequency);

	inline bool ParseIdentifier(const ANSICHAR*& Str, FString& OutStr)
	{
		OutStr = TEXT("");
		FString Result;
		while ((*Str >= 'A' && *Str <= 'Z')
			|| (*Str >= 'a' && *Str <= 'z')
			|| (*Str >= '0' && *Str <= '9')
			|| *Str == '_')
		{
			OutStr += (TCHAR)*Str;
			++Str;
		}

		return OutStr.Len() > 0;
	}

	inline bool ParseIdentifier(const TCHAR*& Str, FString& OutStr)
	{
		OutStr = TEXT("");
		FString Result;
		while ((*Str >= 'A' && *Str <= 'Z')
			|| (*Str >= 'a' && *Str <= 'z')
			|| (*Str >= '0' && *Str <= '9')
			|| *Str == '_')
		{
			OutStr += (TCHAR)*Str;
			++Str;
		}

		return OutStr.Len() > 0;
	}

	inline bool ParseString(const ANSICHAR*& Str, FString& OutStr)
	{
		OutStr = TEXT("");
		FString Result;
		while (*Str != ' ' && *Str != '\n')
		{
			OutStr += (TCHAR)*Str;
			++Str;
		}

		return OutStr.Len() > 0;
	}

	inline bool ParseString(const TCHAR*& Str, FString& OutStr)
	{
		OutStr = TEXT("");
		FString Result;
		while (*Str != ' ' && *Str != '\n')
		{
			OutStr += (TCHAR)*Str;
			++Str;
		}

		return OutStr.Len() > 0;
	}

	FORCEINLINE bool Match(const ANSICHAR*& Str, ANSICHAR Char)
	{
		if (*Str == Char)
		{
			++Str;
			return true;
		}

		return false;
	}

	FORCEINLINE bool Match(const TCHAR*& Str, ANSICHAR Char)
	{
		if (*Str == Char)
		{
			++Str;
			return true;
		}

		return false;
	}

	FORCEINLINE bool Match(const ANSICHAR*& Str, const ANSICHAR* Sub)
	{
		int32 SubLen = FCStringAnsi::Strlen(Sub);
		if (FCStringAnsi::Strncmp(Str, Sub, SubLen) == 0)
		{
			Str += SubLen;
			return true;
		}

		return false;
	}

	FORCEINLINE bool Match(const TCHAR*& Str, const TCHAR* Sub)
	{
		int32 SubLen = FCString::Strlen(Sub);
		if (FCString::Strncmp(Str, Sub, SubLen) == 0)
		{
			Str += SubLen;
			return true;
		}

		return false;
	}

	template <typename T>
	inline bool ParseIntegerNumber(const ANSICHAR*& Str, T& OutNum)
	{
		auto* OriginalStr = Str;
		OutNum = 0;
		while (*Str >= '0' && *Str <= '9')
		{
			OutNum = OutNum * 10 + *Str++ - '0';
		}

		return Str != OriginalStr;
	}

	template <typename T>
	inline bool ParseIntegerNumber(const TCHAR*& Str, T& OutNum)
	{
		auto* OriginalStr = Str;
		OutNum = 0;
		while (*Str >= '0' && *Str <= '9')
		{
			OutNum = OutNum * 10 + *Str++ - '0';
		}

		return Str != OriginalStr;
	}

	inline bool ParseSignedNumber(const ANSICHAR*& Str, int32& OutNum)
	{
		int32 Sign = Match(Str, '-') ? -1 : 1;
		uint32 Num = 0;
		if (ParseIntegerNumber(Str, Num))
		{
			OutNum = Sign * (int32)Num;
			return true;
		}

		return false;
	}

	inline bool ParseSignedNumber(const TCHAR*& Str, int32& OutNum)
	{
		int32 Sign = Match(Str, '-') ? -1 : 1;
		uint32 Num = 0;
		if (ParseIntegerNumber(Str, Num))
		{
			OutNum = Sign * (int32)Num;
			return true;
		}

		return false;
	}

}

#undef UE_API

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CrossCompilerDefinitions.h"
#endif
