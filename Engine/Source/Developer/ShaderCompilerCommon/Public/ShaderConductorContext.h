// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "RHIDefinitions.h"
#include "ShaderCompilerCore.h"
#include "ShaderCore.h"
#include "Templates/Function.h"

#define UE_API SHADERCOMPILERCOMMON_API

class FShaderCompilerDefinitions;

// Cross compiler support/common functionality
namespace CrossCompiler
{

	/** Shader model version for HLSL input language. */
	struct FHlslShaderModel
	{
		/** Major shader model version (e.g. 6 in SM6.2). */
		uint16 Major;

		/** Minor shader model version (e.g. 2 in SM6.2). */
		uint16 Minor;

		FORCEINLINE bool operator == (const FHlslShaderModel& Rhs) const
		{
			return Major == Rhs.Major && Minor == Rhs.Minor;
		}

		FORCEINLINE bool operator != (const FHlslShaderModel& Rhs) const
		{
			return !(*this == Rhs);
		}

		FORCEINLINE bool operator < (const FHlslShaderModel& Rhs) const
		{
			return Major < Rhs.Major || (Major == Rhs.Major && Minor < Rhs.Minor);
		}

		FORCEINLINE bool operator <= (const FHlslShaderModel& Rhs) const
		{
			return *this < Rhs || *this == Rhs;
		}

		FORCEINLINE bool operator > (const FHlslShaderModel& Rhs) const
		{
			return Rhs < *this;
		}

		FORCEINLINE bool operator >= (const FHlslShaderModel& Rhs) const
		{
			return *this > Rhs || *this == Rhs;
		}
	};

	/** Wrapper structure to pass options descriptor to ShaderConductor. This is mapped to <struct ShaderConductor::Compiler::Options>. */
	struct FShaderConductorOptions
	{
		FShaderConductorOptions() = default;

		/** Initializes the options with a call to ApplyCommonCompilerFlags(). */
		FORCEINLINE FShaderConductorOptions(const FShaderCompilerFlags& InCompilerFlags)
		{
			ApplyCommonCompilerFlags(InCompilerFlags);
		}

		/** Removes unused global variables and resources. This can only be used in the HLSL rewrite pass, i.e. 'RewriteHlslSource'. */
		bool bRemoveUnusedGlobals = false;

		/** Experimental: Decide how a matrix get packed. Default in HLSL is row-major. This will be inverted in the SPIR-V backend to match SPIR-V's column-major default. */
		bool bPackMatricesInRowMajor = true;

		/** Enable 16-bit types, such as half, uint16_t. Requires shader model 6.2+. */
		bool bEnable16bitTypes = false;

		/** Embed debug info into the binary. */
		bool bEnableDebugInfo = false;

		/** Force to turn off optimizations. Ignore optimizationLevel below. */
		bool bDisableOptimizations = false;

		/** Enable a pass that converts floating point MUL+ADD pairs into FMAs to avoid re-association. */
		bool bEnableFMAPass = false;

		/** Disables scalar block layout for structured buffers. True for Vulkan mobile due to low coverage of 'VK_EXT_scalar_block_layout' extension. */
		bool bDisableScalarBlockLayout = true;

		/** Enables separate samplers in GLSL via extensions. */
		bool bEnableSeparateSamplersInGlsl = false;

		/** Decorate SV_Position implicitly as invariant. This can drastically reduce Z-fighting but also prevent certain optimizations. */
		bool bSvPositionImplicitInvariant = true;

		/** Decorate output semantics as precise. */
		bool bSupportPreciseOutputs = false;

		/** Preserve storage inputs used for OpenGL */
		bool bPreserveStorageInput = false;
        bool bForceStorageImageFormat = false;

		/** Treat warnigns as errors. This adds '-WX' to the DXC arguments. See CFLAG_WarningsAsErrors. */
		bool bWarningsAsErrors = false;

		enum class ETargetEnvironment
		{
			Vulkan_1_0,
			Vulkan_1_1,
			Vulkan_1_2,
			Vulkan_1_3,
		};
		ETargetEnvironment TargetEnvironment = ETargetEnvironment::Vulkan_1_1;

		/** Shader model version of the input language. By default SM6.2. */
		FHlslShaderModel ShaderModel = { 6, 2 };

		/** HLSL language input version: 2015, 2016, 2017, 2018 (Default), 2021 (Breaking changes in short-circuiting evaluation). */
		uint32 HlslVersion = 2018;

		/**
		 * SPIR-V specific optimization passes to override the default '-O' argument. This will be passed to DXC via the '-Oconfig=...' argument.
		 * Use "preset(relax-nested-expr)" for pre-defined set of optimization passes to relax nested expressions.
		 */
		FString SpirvCustomOptimizationPasses;

		/**
		 * Applies common compiler flags to this options struct: WarningsAsErrors, ForceSpirvDebugInfo, and HLSL2021.
		 * Other flags must be individually applied depending on the shader backend.
		 */
		UE_API void ApplyCommonCompilerFlags(const FShaderCompilerFlags& InCompilerFlags);
	};

	/** Target high level languages for ShaderConductor output. */
	enum class EShaderConductorLanguage
	{
		Hlsl,
		Glsl,
		Essl,
		Metal_macOS,
		Metal_iOS,
	};

	/** Intermediate representation languages for ShaderConductor disassembly output. */
	enum class EShaderConductorIR
	{
		Spirv,
		Dxil,
	};

	/** Shader conductor output target descriptor. */
	struct FShaderConductorTarget
	{
		UE_API FShaderConductorTarget();

		/** Target shader semantics, e.g. "macOS" or "iOS" for Metal GPU semantics. */
		EShaderConductorLanguage Language = EShaderConductorLanguage::Glsl;

		/**
		Target shader version.
		Valid values for HLSL: 50, 60, 61, 62, 63, 64, 65, 66.
		Valid values for Metal family: 20300, 20200, 20100, 20000, 10200, 10100, 10000.
		Valid values for GLSL family: 310, 320, 330, 430.
		*/
		int32 Version = 0;

		/** Cross compilation flags. This is used for high-level cross compilation (such as Metal output) that is send over to SPIRV-Cross, e.g. { "invariant_float_math", "1" }. */
		TPimplPtr<FShaderCompilerDefinitions> CompileFlags;

		/** Optional callback to rename certain variable types. */
		TFunction<bool(const FAnsiStringView& VariableName, const FAnsiStringView& TypeName, FString& OutRenamedTypeName)> VariableTypeRenameCallback;
	};

	/** Container for all special case SPIR-V identifiers generated by ShaderConductor. */
	struct FShaderConductorIdentifierTable
	{
		/** Identifier for vertex input attributes: "in.var.ATTRIBUTE". */
		const ANSICHAR* InputAttribute;

		/** Identifier for globals uniform buffers: "$Globals". */
		const ANSICHAR* GlobalsUniformBuffer;

		/** Identifier for the intermediate output variable in a tessellation-control shader. */
		const ANSICHAR* IntermediateTessControlOutput;

		/** Identifier for dummy samplers used for platforms where samplers are required */
		const ANSICHAR* DummySampler;
	};

	/** Wrapper class to handle interface between UE and ShaderConductor. Use to compile HLSL shaders to SPIR-V or high-level languages such as Metal. */
	class FShaderConductorContext
	{
	public:
		/** Initializes the context with internal buffers used for the conversion of input and option descriptors between UE and ShaderConductor. */
		UE_API FShaderConductorContext();

		/** Release the internal buffers. */
		UE_API ~FShaderConductorContext();

		/** Move constructor to take ownership of internal buffers from 'Rhs'. */
		UE_API FShaderConductorContext(FShaderConductorContext&& Rhs);

		/** Move operator to take ownership of internal buffers from 'Rhs'. */
		UE_API FShaderConductorContext& operator = (FShaderConductorContext&& Rhs);

		FShaderConductorContext(const FShaderConductorContext&) = delete;
		FShaderConductorContext& operator = (const FShaderConductorContext&) = delete;

		/** Loads the shader source and converts the input descriptor to a format suitable for ShaderConductor. If 'Definitions' is null, the previously loaded definitions are not modified. */
		UE_API bool LoadSource(FStringView ShaderSource, const FString& Filename, const FString& EntryPoint, EShaderFrequency ShaderStage, const FShaderCompilerDefinitions* Definitions = nullptr, const TArray<FString>* ExtraDxcArgs = nullptr);
		UE_API bool LoadSource(FAnsiStringView ShaderSource, const FString& Filename, const FString& EntryPoint, EShaderFrequency ShaderStage, const FShaderCompilerDefinitions* Definitions = nullptr, const TArray<FString>* ExtraDxcArgs = nullptr);
		UE_API bool LoadSource(const ANSICHAR* ShaderSource, const ANSICHAR* Filename, const ANSICHAR* EntryPoint, EShaderFrequency ShaderStage, const FShaderCompilerDefinitions* Definitions = nullptr, const TArray<FString>* ExtraDxcArgs = nullptr);

        /** Compiles the specified HLSL shader source code to DXIL. */
        UE_API bool CompileHlslToDxil(const FShaderConductorOptions& Options, TArray<uint32>& OutDxil);
        
		/** Compiles the specified HLSL shader source code to SPIR-V. */
		UE_API bool CompileHlslToSpirv(const FShaderConductorOptions& Options, TArray<uint32>& OutSpirv);

		/** Replaces #line 123 directives with //ine 123. Required to work around platform-specific shader debug data handling issues. */
		UE_API void RemoveLineDirectives();

		/** Performs the specified optimization passes (e.g. "-O" or "--strip-reflect") on the SPIR-V module. */
		UE_API bool OptimizeSpirv(TArray<uint32>& Spirv, const ANSICHAR* const * OptConfigs, int32 OptConfigCount);

		/** Compiles the specified SPIR-V shader binary code to high level source code (Metal or GLSL). */
		UE_API bool CompileSpirvToSource(const FShaderConductorOptions& Options, const FShaderConductorTarget& Target, const void* InSpirv, uint32 InSpirvByteSize, FString& OutSource);

		/** Compiles the specified SPIR-V shader binary code to high level source code (Metal or GLSL) stored as null terminated ANSI string. */
		UE_API bool CompileSpirvToSourceAnsi(const FShaderConductorOptions& Options, const FShaderConductorTarget& Target, const void* InSpirv, uint32 InSpirvByteSize, TArray<ANSICHAR>& OutSource);

		/** Compiles the specified SPIR-V shader binary code to high level source code (Metal or GLSL) stored as byte buffer (without null terminator as it comes from ShaderConductor). */
		UE_API bool CompileSpirvToSourceBuffer(const FShaderConductorOptions& Options, const FShaderConductorTarget& Target, const void* InSpirv, uint32 InSpirvByteSize, const TFunction<void(const void* Data, uint32 Size)>& OutputCallback);

		/** Flushes the list of current compile errors and moves the ownership to the caller. */
		UE_API void FlushErrors(TArray<FShaderCompilerError>& OutErrors);

		/** Returns a pointer to a null terminated ANSI string of the internal loaded sources, or null if no source has been loaded yet. This is automatically updated when RewriteHlsl() is called. */
		UE_API const ANSICHAR* GetSourceString() const;

		/** Returns a length of the internal loaded sources (excluding the null terminator). This is automatically updated when RewriteHlsl() is called. */
		UE_API int32 GetSourceLength() const;

		/** Returns the DXC command line arguments for the specified options. This does not include an output file, i.e. "-Fo" argument is not included. */
		UE_API FString GenerateDxcArguments(const FShaderConductorOptions& Options) const;

		/** Returns the list of current compile errors. */
		inline const TArray<FShaderCompilerError>& GetErrors() const
		{
			return Errors;
		}

	public:
		/** Convert raw ANSI string buffer containing DXC errors into array of <FShaderCompilerError>. */
		static UE_API void ConvertCompileErrors(FAnsiStringView ErrorsString, TArray<FShaderCompilerError>& OutErrors);

		/** Returns whether the specified variable name denotes an intermediate output variable.
		This is only true for a special identifiers generated by DXC to communicate patch constant data in the Hull Shader. */
		static UE_API bool IsIntermediateSpirvOutputVariable(const ANSICHAR* SpirvVariableName);

		/** Returns the table of special identifiers generated by ShaderConductor. */
		static UE_API const FShaderConductorIdentifierTable& GetIdentifierTable();

		/** Disassembles the specified SPIR-V or DXIL module and returns its assembly as text representation. */
		static UE_API bool Disassemble(EShaderConductorIR Language, const void* Binary, uint32 BinaryByteSize, TArray<ANSICHAR>& OutAssemblyText);

		/** Helper function to disassemble the specified SPIR-V or DXIL module and add it to the output code reflection array. */
		static UE_API bool Disassemble(EShaderConductorIR Language, const void* Binary, uint32 BinaryByteSize, FGenericShaderStat& OutShaderStat);

		/** Returns a filename extension for the specified shading language and shader stage, e.g. "frag" for a GLSL pixel shader. */
		static UE_API const TCHAR* GetShaderFileExt(EShaderConductorLanguage Language, EShaderFrequency ShaderStage = SF_NumFrequencies);

		/** Explicitly shut down ShaderConductor and DXC shared libraries. Only used for Linux to prevent dangling mutex on exit. */
		static UE_API void Shutdown();

	public:
		struct FShaderConductorIntermediates; // Pimpl idiom

	private:
		TArray<FShaderCompilerError> Errors;
		FShaderConductorIntermediates* Intermediates; // Pimpl idiom
	};

}

#undef UE_API

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CrossCompilerDefinitions.h"
#endif
