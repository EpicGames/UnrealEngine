// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Containers/AnsiString.h"
#include "Containers/UnrealString.h"
#include "Misc/EnumClassFlags.h"
#include "ShaderSource.h"

namespace UE::ShaderMinifier
{

enum EMinifyShaderFlags
{
	None = 0,

	/** Add `// REASON: foo` comments next to emitted code blocks to identify which other block uses it or other reason why it was included */
	OutputReasons  = 1 << 1,

	/** Add a comment describing how many functions, structs, etc. were emitted */
	OutputStats    = 1 << 2,

	/** Add `#line <line> <filename>` directives for each emitted code block */
	OutputLines    = 1 << 3,

	/** Keep original global scope comment lines, such as `// #define FOO 123` that were added by UE shader preprocessor */
	OutputCommentLines = 1 << 4,
};
ENUM_CLASS_FLAGS(EMinifyShaderFlags)

struct FDiagnosticMessage
{
	FString Message;
	FString File;
	int32 Offset  = INDEX_NONE;
	int32 Line    = INDEX_NONE;
	int32 Column  = INDEX_NONE;
};

struct FDiagnostics
{
	TArray<FDiagnosticMessage> Errors;
	TArray<FDiagnosticMessage> Warnings;
};

struct FMinifiedShader
{
	FShaderSource::FStringType Code;
	FDiagnostics Diagnostics;

	bool Success() const
	{
		return Diagnostics.Errors.IsEmpty() && !Code.IsEmpty();
	}
};

extern SHADERCOMPILERCOMMON_API FMinifiedShader Minify(const FShaderSource& PreprocessedShader, const FShaderSource::FViewType EntryPoint, EMinifyShaderFlags Flags = EMinifyShaderFlags::None);
extern SHADERCOMPILERCOMMON_API FMinifiedShader Minify(const FShaderSource& PreprocessedShader, TConstArrayView<FShaderSource::FViewType> RequiredSymbols, EMinifyShaderFlags Flags = EMinifyShaderFlags::None);
} // UE::ShaderMinifier

