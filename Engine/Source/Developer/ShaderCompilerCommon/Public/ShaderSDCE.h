// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ShaderPreprocessTypes.h"

namespace UE::ShaderMinifier::SDCE
{
	/**
	 * Minify shader sources
	 */
	SHADERCOMPILERCOMMON_API void MinifyInPlace(const TConstArrayView<FStringView>& DCESymbols, FString& Code);
	
	/**
	 * Minify shader sources from preprocessor output
	 */
	SHADERCOMPILERCOMMON_API bool MinifyInPlace(const FShaderCompilerInput& Input, const FShaderPreprocessOutput& PreprocessOutput, FString& Code);
}
