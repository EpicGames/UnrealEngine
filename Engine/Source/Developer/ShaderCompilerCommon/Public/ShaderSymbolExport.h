// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Bad includes (ZipArchiveWriter.h)

#if WITH_ENGINE

#include "Tasks/Task.h"
#include "Compression/CompressedBuffer.h"
#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "HAL/CriticalSection.h"
#include "HAL/Event.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/ScopeLock.h"
#include "Serialization/MemoryReader.h"
#include "Templates/UniquePtr.h"

#include <atomic>

class FZipArchiveWriter;

class FShaderSymbolExport
{
public:
	FShaderSymbolExport() = delete;
	SHADERCOMPILERCOMMON_API FShaderSymbolExport(FName InShaderFormat);
	SHADERCOMPILERCOMMON_API ~FShaderSymbolExport();

	/** Should be called from IShaderFormat::NotifyShaderCompiled implementation.
	*   Template type is the platform specific symbol data structure.
	*/
	template<typename TPlatformShaderSymbolData>
	UE_DEPRECATED(5.6, "Use overload accepting an FCompressedBuffer for symbol data.")
	void NotifyShaderCompiled(const TConstArrayView<uint8>& PlatformSymbolData, const FString& DebugInfo = FString()) {}

	template<typename TPlatformShaderSymbolData>
	void NotifyShaderCompiled(const FCompressedBuffer& PlatformSymbolDataCompressed, const FString& DebugInfo = FString());

	/** Called at the end of a cook to free resources and finalize artifacts created during the cook. */
	SHADERCOMPILERCOMMON_API void NotifyShaderCompilersShutdown();

private:
	SHADERCOMPILERCOMMON_API void Initialize();
	SHADERCOMPILERCOMMON_API void WriteSymbolData(const FString& Filename, const FString& DebugInfo, TConstArrayView<uint8> Contents);
	void WriteFile(const FString& Filename, const TConstArrayView<uint8>& Contents, bool bAllowRetry);

	const FName ShaderFormat;

	TUniquePtr<FZipArchiveWriter> ZipWriter;
	TSet<FString> ExportedShaders;
	struct FSymbolFileInfo
	{
		uint64 Hash;
		int32 Size;
	};
	TMap<FString, FSymbolFileInfo> ExportedSymbolInfo;
	FString ExportPath;
	FString InfoFilePath;
	FString ExportFileName;
	uint64 TotalSymbolDataBytes{ 0 };
	uint64 TotalSymbolData{ 0 };
	bool bExportShaderSymbols{ false };

	TMap<FString, FString> ShaderInfos;

	std::atomic<uint32> DuplicateSymbols{ 0 };

	/**
	 * If true, the current process is the first process in a multiprocess group, or is not in a group,
	 * and should combine artifacts produced by the other processes. Will also be false if no combination
	 * is necessary for given settings.
	 */
	bool bMultiprocessOwner{ false };

	std::atomic<bool> bInitialized{ false };
	FCriticalSection InitCs;

	// Write task synchronization: tasks run concurrently for decompress/deserialize;
	// PendingTaskCount tracks in-flight tasks; AllTasksDoneEvent is triggered when it reaches
	// zero so NotifyShaderCompilersShutdown can drain without holding references to each task.
	FRWLock SymbolWriteLock;
	std::atomic<int32> PendingTaskCount{ 0 };
	FEventRef AllTasksDoneEvent{ EEventMode::ManualReset };
};

template<typename TPlatformShaderSymbolData>
inline void FShaderSymbolExport::NotifyShaderCompiled(const FCompressedBuffer& PlatformSymbolDataCompressed, const FString& DebugInfo)
{
	// Double-checked locking: atomic acquire load on the fast path, InitCs taken only on first call
	if (!bInitialized.load(std::memory_order_acquire))
	{
		FScopeLock Lock(&InitCs);
		if (!bInitialized.load(std::memory_order_relaxed))
		{
			// If we get called, we know we're compiling. Do one time initialization
			// which will create the output directory / open the output file stream.
			Initialize();
			bInitialized.store(true, std::memory_order_release);
		}
	}

	if (!bExportShaderSymbols) // read-only after init; safe via acquire above
	{
		return;
	}

	// Dispatch a task to decompress, deserialize, and write. All heavy work off the calling thread.
	// The counter is incremented before launch; the task decrements it on completion and triggers
	// AllTasksDoneEvent when it reaches zero so shutdown can drain without holding task references.
	PendingTaskCount.fetch_add(1, std::memory_order_relaxed);
	
	UE::Tasks::Launch(TEXT("ShaderSymbolExport::WriteSymbolData"),
		[this, PlatformSymbolDataCompressed, DebugInfo]()
		{
			TPlatformShaderSymbolData FullSymbolData;
			FSharedBuffer PlatformSymbolData = PlatformSymbolDataCompressed.Decompress();
			FMemoryReaderView Ar(PlatformSymbolData.GetView());
			Ar << FullSymbolData;

			for (const auto& SymbolData : FullSymbolData.GetAllSymbolData())
			{
				WriteSymbolData(SymbolData.GetFilename(), DebugInfo, SymbolData.GetContents());
			}

			if (PendingTaskCount.fetch_sub(1, std::memory_order_acq_rel) == 1)
			{
				AllTasksDoneEvent->Trigger();
			}
		},
		UE::Tasks::ETaskPriority::BackgroundLow);
}

#endif // WITH_ENGINE
