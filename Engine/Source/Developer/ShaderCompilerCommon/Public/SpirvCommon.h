// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"

THIRD_PARTY_INCLUDES_START
	#include <spirv/unified1/spirv.h>
THIRD_PARTY_INCLUDES_END


/** Template forward iterator for SPIR-V instructions. */
template <typename T>
class TSpirvForwardIterator
{
public:
	/** Initializes the iterator with a null pointer. */
	TSpirvForwardIterator() :
		Ptr(nullptr)
	{
	}

	TSpirvForwardIterator(const TSpirvForwardIterator& Other) = default;
	TSpirvForwardIterator& operator = (const TSpirvForwardIterator& Other) = default;

	/** Initializes the iterator with the specified pointer. */
	TSpirvForwardIterator(T* InPtr, bool bPointsAtHeader = false) :
		Ptr(InPtr)
	{
		// Ignore null pointers so that empty SPIR-V modules can still use begin()/end() loops with zero iterations.
		if (bPointsAtHeader && Ptr != nullptr)
		{
			// Validate header of SPIR-V module
			const uint32 MagicNumberValue = Ptr[0];
			check(MagicNumberValue == SpvMagicNumber);

			//const uint32 Version = Ptr[1];
			//const uint32 Generator = Ptr[2];
			//const uint32 Bound = Ptr[3];
			const uint32 ZeroCheckValue = Ptr[4];
			checkf(ZeroCheckValue == 0, TEXT("redundancy check for SPIR-V module failed: WORD[4] is non-zero"));

			// Move pointer to first instruction
			Ptr += 5;
		}
	}

	/** Returns the opcode of the current instruction. */
	SpvOp Opcode() const
	{
		return static_cast<SpvOp>(*Ptr & SpvOpCodeMask);
	}

	/** Returns the word count of the current instruction. A valid SPIR-V instruction must have a word count greater zero. */
	uint32 WordCount() const
	{
		return (*Ptr >> SpvWordCountShift) & SpvOpCodeMask;
	}

	/** Dereferences the value the current operand points to. */
	uint32 Operand(int32 WordOffset) const
	{
		return Ptr[WordOffset];
	}

	/** Returns the operand as the specified reinterpreted type. */
	template <typename TDst>
	TDst OperandAs(int32 WordOffset) const
	{
		return *reinterpret_cast<const TDst*>(&(Ptr[WordOffset]));
	}

	/** Returns the operand pointer as ANSI C string. */
	const ANSICHAR* OperandAsString(int32 WordOffset) const
	{
		return reinterpret_cast<const ANSICHAR*>(&(Ptr[WordOffset]));
	}

	/** Returns a pointer to the beginning of the SPIR-V instruction this iterator currently points to. */
	T* operator * () const
	{
		return Ptr;
	}

	/** Increments the iterator to point to the next SPIR-V instruction. */
	TSpirvForwardIterator& operator ++ ()
	{
		const uint32 Words = WordCount();
		checkf(Words > 0, TEXT("Invalid SPIR-V instruction with word count of zero"));
		Ptr += Words;
		return *this;
	}

	/** Increments the iterator to point to the next SPIR-V instruction and returns the previous iterator state. */
	TSpirvForwardIterator operator ++ (int)
	{
		TSpirvForwardIterator Current(*this);
		this->operator++();
		return Current;
	}

	/** Returns whether this iterator points to the same address as the other iterator. */
	bool operator == (const TSpirvForwardIterator& Other) const
	{
		return Ptr == Other.Ptr;
	}

	/** Returns whether this iterator does not point to the same address as the other iterator. */
	bool operator != (const TSpirvForwardIterator& Other) const
	{
		return Ptr != Other.Ptr;
	}

private:
	T* Ptr;
};

using FSpirvIterator = TSpirvForwardIterator<uint32>;
using FSpirvConstIterator = TSpirvForwardIterator<const uint32>;

/** Base structure for SPIR-V modules in the shader backends. */
struct FSpirv
{
	TArray<uint32> Data;

	/** Returns a byte pointer to the SPIR-V data. */
	FORCEINLINE int8* GetByteData()
	{
		return reinterpret_cast<int8*>(Data.GetData());
	}

	/** Returns a byte pointer to the SPIR-V data. */
	FORCEINLINE const int8* GetByteData() const
	{
		return reinterpret_cast<const int8*>(Data.GetData());
	}

	/** Returns the size of this SPIR-V module in bytes. */
	FORCEINLINE int32 GetByteSize() const
	{
		return Data.Num() * sizeof(uint32);
	}

	/** Returns the word offset to the instruction the specified iterator points to plus additional operand word offset. */
	FORCEINLINE uint32 GetWordOffset(const FSpirvConstIterator& Iter, uint32 OperandWordOffset = 0) const
	{
		return static_cast<uint32>(*Iter + OperandWordOffset - Data.GetData());
	}

public:
	/** Returns a constant iterator to the first instruction in this SPIR-V module. */
	FORCEINLINE FSpirvConstIterator cbegin() const
	{
		return FSpirvConstIterator(Data.GetData(), /*bPointsAtHeader:*/ true);
	}

	/** Returns a constant iterator to the first instruction in this SPIR-V module. */
	FORCEINLINE FSpirvConstIterator begin() const
	{
		return FSpirvConstIterator(Data.GetData(), /*bPointsAtHeader:*/ true);
	}

	/** Returns an iterator to the first instruction in this SPIR-V module. */
	FORCEINLINE FSpirvIterator begin()
	{
		return FSpirvIterator(Data.GetData(), /*bPointsAtHeader:*/ true);
	}

	/** Returns a constant iterator to the end of this SPIR-V module. */
	FORCEINLINE FSpirvConstIterator cend() const
	{
		return FSpirvConstIterator(Data.GetData() + Data.Num());
	}

	/** Returns a constant iterator to the end of this SPIR-V module. */
	FORCEINLINE FSpirvConstIterator end() const
	{
		return FSpirvConstIterator(Data.GetData() + Data.Num());
	}

	/** Returns an iterator to the end of this SPIR-V module. */
	FORCEINLINE FSpirvIterator end()
	{
		return FSpirvIterator(Data.GetData() + Data.Num());
	}
};

/** Returns the string representation of the specified SPIR-V built-in variable, e.g. SpvBuiltInPosition to TEXT("gl_Position"). Returns nullptr for invalid value. */
SHADERCOMPILERCOMMON_API const TCHAR* SpirvBuiltinToString(const SpvBuiltIn BuiltIn);

/** Returns word offset to the entry point (OpEntryPoint) and name of the entry point (OpName). If the respective entry has not been found, the output offset is 0. */
SHADERCOMPILERCOMMON_API void FindOffsetToSpirvEntryPoint(const FSpirv& Spirv, const ANSICHAR* EntryPointName, uint32& OutWordOffsetToEntryPoint, uint32& OutWordOffsetToMainName);

/** Finds the entry point name name. */
extern SHADERCOMPILERCOMMON_API const ANSICHAR* FindSpirvEntryPoint(const FSpirv& Spirv);

/** Renames the fixed-size entry point name (which must be "main_00000000_00000000") to the formatted name including a CRC over the module. Returns the new entry point name. */
SHADERCOMPILERCOMMON_API const ANSICHAR* PatchSpirvEntryPointWithCRC(FSpirv& Spirv, uint32& OutCRC);

/** Parse global variables (anything but function local variables) of specified storage class from the specified SPIR-V module. */
SHADERCOMPILERCOMMON_API void ParseSpirvGlobalVariables(const FSpirv& Spirv, SpvStorageClass StorageClass, TArray<FString>& OutVariableNames);

/** Parse global variables (anything but function local variables) of specified storage class from the specified SPIR-V module.  Currently only supports plain data (integers, floats, vectors). */
struct FSpirvStructureMemberData
{
	FString Name;
	int32 Offset = -1;
	uint32 Size = 0;
};
SHADERCOMPILERCOMMON_API TArray<FSpirvStructureMemberData> ParseSpirvStructureMembers(const FSpirv& Spirv, FAnsiStringView StructureName);
