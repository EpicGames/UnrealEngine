// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"

#if PLATFORM_MAC || PLATFORM_WINDOWS || PLATFORM_LINUX
THIRD_PARTY_INCLUDES_START
	#include <spirv/unified1/spirv.h>
	#include "spirv_reflect.h"
THIRD_PARTY_INCLUDES_END
#endif


/** Container structure for all SPIR-V reflection resources and in/out attributes. */
struct FSpirvReflectBindings
{
	TArray<SpvReflectInterfaceVariable*> InputAttributes;
	TArray<SpvReflectInterfaceVariable*> OutputAttributes;
	TArray<SpvReflectInterfaceVariable*> PushConstants;
	TSet<SpvReflectDescriptorBinding*> AtomicCounters;
	TArray<SpvReflectDescriptorBinding*> InputAttachments; // for subpass inputs
	TArray<SpvReflectDescriptorBinding*> UniformBuffers;
	TArray<SpvReflectDescriptorBinding*> Samplers;
	TArray<SpvReflectDescriptorBinding*> TextureSRVs;
	TArray<SpvReflectDescriptorBinding*> TextureUAVs;
	TArray<SpvReflectDescriptorBinding*> TBufferSRVs;
	TArray<SpvReflectDescriptorBinding*> TBufferUAVs;
	TArray<SpvReflectDescriptorBinding*> SBufferSRVs;
	TArray<SpvReflectDescriptorBinding*> SBufferUAVs;
	TArray<SpvReflectDescriptorBinding*> AccelerationStructures;
	int32 NumBindlessSamplers = 0;
	int32 NumBindlessResources = 0;

	/** Adds the specified descriptor binding to the corresponding container iff the descriptor is marked as being accessed. */
	SHADERCOMPILERCOMMON_API void AddDescriptorBinding(SpvReflectDescriptorBinding* InBinding);

	/** Gathers all descriptor bindings from the specified SPIRV-Reflect module. */
	SHADERCOMPILERCOMMON_API void GatherDescriptorBindings(const spv_reflect::ShaderModule& SpirvReflection);

	/** Gathers all input interface variables from the specified SPIRV-Reflect module. */
	SHADERCOMPILERCOMMON_API void GatherInputAttributes(const spv_reflect::ShaderModule& SpirvReflection);

	/** Gathers all output interface variables from the specified SPIRV-Reflect module. */
	SHADERCOMPILERCOMMON_API void GatherOutputAttributes(const spv_reflect::ShaderModule& SpirvReflection);

	/** Gathers all push constant variables from the specified SPIRV-Reflect module. */
	SHADERCOMPILERCOMMON_API void GatherPushConstants(const spv_reflect::ShaderModule& SpirvReflection);

	/** Assigns the binding location for all input attributes by its semantic index if their name is equal to 'SemanticName'. Used for vertex shaders where all input attributes have the "ATTRIBUTE" semantic. */
	SHADERCOMPILERCOMMON_API void AssignInputAttributeLocationsBySemanticIndex(spv_reflect::ShaderModule& SpirvReflection, const ANSICHAR* SemanticName);
};



