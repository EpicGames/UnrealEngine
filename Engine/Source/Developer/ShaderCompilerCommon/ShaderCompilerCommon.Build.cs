// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class ShaderCompilerCommon : ModuleRules
{
	public ShaderCompilerCommon(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"RenderCore",
				"ShaderPreprocessor",
			}
			);
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"HlslParser",
			}
			);
		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"TargetPlatform",
			}
			);
		if (Target.bCompileAgainstEngine)
		{
			PublicDependencyModuleNames.AddRange(
			new string[] {
				"FileUtilities",
			}
			);
		}

		if (Target.Platform == UnrealTargetPlatform.Mac || Target.Platform == UnrealTargetPlatform.Win64 || Target.Platform == UnrealTargetPlatform.Linux)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "ShaderConductor");
			AddEngineThirdPartyPrivateStaticDependencies(Target, "SPIRVReflect");
		}

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			PublicSystemLibraries.Add("ImageHlp.lib");
		}

		// We only need a header containing definitions
		PublicSystemIncludePaths.Add(Path.Combine(Target.UEThirdPartySourceDirectory, "SPIRV-Reflect/SPIRV-Reflect"));
		PublicSystemIncludePaths.Add(Path.Combine(Target.UEThirdPartySourceDirectory, "ShaderConductor/ShaderConductor/External/SPIRV-Headers/include"));
	}
}
